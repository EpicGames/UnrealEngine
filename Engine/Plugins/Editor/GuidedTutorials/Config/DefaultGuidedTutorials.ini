[/Script/IntroTutorials.EditorTutorialSettings]
+Categories=(Identifier="Basics",Title=NSLOCTEXT("TutorialCategories","BasicsTitle","Basics"),Description=NSLOCTEXT("TutorialCategories","BasicsDescription","Getting started with Unreal Engine."),Icon="PlayWorld.RepeatLastPlay",Texture=None,SortOrder=100)
+Categories=(Identifier="Blueprints",Title=NSLOCTEXT("TutorialCategories","BlueprintsTitle","Blueprints"),Description=NSLOCTEXT("TutorialCategories","BlueprintsDescription","Tutorials covering the creation and usage of Blueprints."),Icon="FullBlueprintEditor.EditGlobalOptions",Texture=/GuidedTutorials/Tutorial/BlueprintTutorials/TutorialAssets/Blueprint_64x.Blueprint_64x,SortOrder=200)
+Categories=(Identifier="Editors",Title=NSLOCTEXT("TutorialCategories","EditorsTitle","Editors"),Description=NSLOCTEXT("TutorialCategories","EditorsDescription","Tutorials covering the various sub editors of Unreal Engine."),Icon=,Texture=/GuidedTutorials/Tutorial/SubEditors/TutorialAssets/icon_Editor_Preferences_General_40x.icon_Editor_Preferences_General_40x,SortOrder=225)
+Categories=(Identifier="Code",Title=NSLOCTEXT("TutorialCategories","CodeTitle","Code"),Description=NSLOCTEXT("TutorialCategories","CodeDescription","Write C++ code for use in Unreal Engine."),Icon="PlayWorld.RepeatLastPlay",Texture=None,SortOrder=250)
+Categories=(Identifier="Animation",Title=NSLOCTEXT("TutorialCategories","AnimationTitle","Animation"),Description=NSLOCTEXT("TutorialCategories","AnimationDescription","Tutorials covering the animation system in Unreal Engine."),Icon=,Texture=/GuidedTutorials/Tutorial/SubEditors/TutorialAssets/icon_ShowSkeletalMeshes_40x.icon_ShowSkeletalMeshes_40x,SortOrder=600)
+Categories=(Identifier="Landscape",Title=NSLOCTEXT("TutorialCategories","LandscapeTitle","Landscape"),Description=NSLOCTEXT("TutorialCategories","LandscapeDescription","Tutorials covering the Unreal Editor terrain editor: Landscape."),Icon="LevelEditor.LandscapeMode",Texture=/GuidedTutorials/Tutorial/Landscape/TutorialAssets/Landscape.Landscape,SortOrder=700)
+Categories=(Identifier="Foliage",Title=NSLOCTEXT("TutorialCategories","FoliageTitle","Foliage"),Description=NSLOCTEXT("TutorialCategories","FoliageDescription","Tutorials covering the Unreal Engine Foliage tool."),Icon="LevelEditor.FoliageMode",Texture=/GuidedTutorials/Tutorial/Foliage/TutorialAssets/Foliage.Foliage,SortOrder=800)
+Categories=(Identifier="Mobile",Title=NSLOCTEXT("TutorialCategories","MobileTitle","Mobile"),Description=NSLOCTEXT("TutorialCategories","MobileDescription","Mobile Tutorials."),Icon="MaterialEditor.TogglePlatformStats",Texture=None,SortOrder=900)

+StartupTutorial=/GuidedTutorials/Tutorial/Basics/LevelEditorAttract.LevelEditorAttract_C

+TutorialContexts=(Context="StaticMeshEditor",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/SubEditors/StaticMeshEditorTutorial.StaticMeshEditorTutorial_C)
+TutorialContexts=(Context="LevelEditor",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/Basics/LevelEditorOverview.LevelEditorOverview_C)
+TutorialContexts=(Context="BlueprintEditor",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/BlueprintTutorials/BlueprintEditorTutorial.BlueprintEditorTutorial_C)
+TutorialContexts=(Context="BlueprintEditor.MacroLibrary",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/BlueprintTutorials/BlueprintMacroLibrariesEditorOverview.BlueprintMacroLibrariesEditorOverview_C)
+TutorialContexts=(Context="BlueprintEditor.Interface",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/BlueprintTutorials/BlueprintInterfacesEditorOverview.BlueprintInterfacesEditorOverview_C)
+TutorialContexts=(Context="BlueprintEditor.LevelScript",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/BlueprintTutorials/LevelBlueprintEditorOverview.LevelBlueprintEditorOverview_C)
+TutorialContexts=(Context="MaterialEditor",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/SubEditors/MaterialEditorTutorial.MaterialEditorTutorial_C)
+TutorialContexts=(Context="Cascade",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/SubEditors/ParticleSystemEditorTutorial.ParticleSystemEditorTutorial_C)
+TutorialContexts=(Context="Persona",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/SubEditors/PersonaAnimEditorWalkThrough.PersonaAnimEditorWalkthrough_C)
+TutorialContexts=(Context="LandscapeMode",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/Landscape/Landscape_Manage_Mode.Landscape_Manage_Mode_C)
+TutorialContexts=(Context="FoliageMode",BrowserFilter=,AttractTutorial=None,LaunchTutorial=/GuidedTutorials/Tutorial/Foliage/Foliage_Intro_Tutorial.Foliage_Intro_Tutorial_C)

