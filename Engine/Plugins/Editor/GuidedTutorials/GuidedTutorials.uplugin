{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Guided Tutorials",
	"Description": "Adds classes and content that support running guided tutorials within the editor UI.",
	"Category": "Learning",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"CanContainVerse": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "IntroTutorials",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
