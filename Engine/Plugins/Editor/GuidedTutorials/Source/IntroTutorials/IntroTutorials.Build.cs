// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class IntroTutorials : ModuleRules
	{
		public IntroTutorials(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine", // @todo Mac: for some reason CoreUObject and Engine are needed to link in debug on Mac
                    "InputCore",
                    "Slate",
					"SlateCore",
                    "Documentation",
					"GraphEditor",
					"BlueprintGraph",
					"MessageLog",
					"ApplicationCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
                    "AppFramework",
					"EditorFramework",
					"UnrealEd",
                    "Kismet",
                    "PlacementMode",
					"SlateCore",
					"Settings",
					"PropertyEditor",
					"DesktopPlatform",
					"AssetTools",
					"SourceCodeAccess",
					"ContentBrowser",
					"LevelEditor",
                    "AssetRegistry",
					"Analytics",
					"ToolMenus",
					"GameProjectGeneration",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"MainFrame",
					"TargetPlatform",
					"TargetDeviceServices",
					"LauncherServices",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"MainFrame",
					"LauncherServices",
				}
			);
		}
	}
}
