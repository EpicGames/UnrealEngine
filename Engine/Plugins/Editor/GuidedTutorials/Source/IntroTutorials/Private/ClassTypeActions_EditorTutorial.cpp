// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassTypeActions_EditorTutorial.h"
#include "Layout/Margin.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "IIntroTutorials.h"
#include "EditorTutorial.h"
#include "EditorTutorialStyle.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "IntroTutorials"

UClass* FClassTypeActions_EditorTutorial::GetSupportedClass() const
{
	return UEditorTutorial::StaticClass();
}

TSharedPtr<SWidget> FClassTypeActions_EditorTutorial::GetThumbnailOverlay(const FAssetData& AssetData) const
{
	const FString FullTutorialAssetPath = AssetData.GetObjectPathString();
	auto OnLaunchTutorialClicked = [FullTutorialAssetPath]() -> FReply
	{
		if (IIntroTutorials::IsAvailable())
		{
			IIntroTutorials& IntroTutorials = IIntroTutorials::Get();
			IntroTutorials.LaunchTutorial(FullTutorialAssetPath);
		}
		return FReply::Handled();
	};

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2))
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("Blueprint_LaunchTutorialToolTip", "Launch this tutorial."))
			.Cursor(EMouseCursor::Default) // The outer widget can specify a DragHand cursor, so we need to override that here
			.ForegroundColor(FSlateColor::UseForeground())
			.OnClicked_Lambda(OnLaunchTutorialClicked)
			[
				SNew(SBox)
				.MinDesiredWidth(16)
				.MinDesiredHeight(16)
				[
					SNew(SImage)
					.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.PlayButton.Image"))
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
