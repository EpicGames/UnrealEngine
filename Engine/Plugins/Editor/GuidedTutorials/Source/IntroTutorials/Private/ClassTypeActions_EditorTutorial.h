// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "ClassTypeActions_Base.h"

struct FAssetData;

class FClassTypeActions_EditorTutorial : public FClassTypeActions_Base
{
public:
	// IClassTypeActions Implementation
	virtual UClass* GetSupportedClass() const override;
	virtual TSharedPtr<class SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;
};
