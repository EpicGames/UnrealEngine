// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTutorial.h"
#include "Modules/ModuleManager.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Framework/Docking/TabManager.h"
#include "GameFramework/Actor.h"
#include "Settings/ContentBrowserSettings.h"
#include "EngineGlobals.h"
#include "Editor.h"

#include "IIntroTutorials.h"
#include "IntroTutorials.h"
#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Misc/RuntimeErrors.h"
#include "Subsystems/AssetEditorSubsystem.h"

#if WITH_EDITORONLY_DATA
namespace
{
	void GatherEditorTutorialForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const UEditorTutorial* const EditorTutorial = CastChecked<UEditorTutorial>(Object);

		// Editor Tutorial assets never exist at runtime, so treat all of their properties and script data as editor-only (as they may be derived from by a blueprint)
		PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(EditorTutorial, GatherTextFlags | EPropertyLocalizationGathererTextFlags::ForceEditorOnly);
	}
}
#endif

UEditorTutorial::UEditorTutorial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UEditorTutorial::StaticClass(), &GatherEditorTutorialForLocalization); }
#endif
}

UWorld* UEditorTutorial::GetWorld() const
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		return GWorld;
	}
#endif // WITH_EDITOR
	return GEngine->GetWorldContexts()[0].World();
}

void UEditorTutorial::GoToNextTutorialStage()
{
	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.GoToNextStage(nullptr);
}


void UEditorTutorial::GoToPreviousTutorialStage()
{
	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.GoToPreviousStage();
}


void UEditorTutorial::BeginTutorial(UEditorTutorial* TutorialToStart, bool bRestart)
{
	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.LaunchTutorial(TutorialToStart, bRestart ? IIntroTutorials::ETutorialStartType::TST_RESTART : IIntroTutorials::ETutorialStartType::TST_CONTINUE);
}


void UEditorTutorial::HandleTutorialStageStarted(FName StageName)
{
	FEditorScriptExecutionGuard ScriptGuard;
	OnTutorialStageStarted(StageName);
}


void UEditorTutorial::HandleTutorialStageEnded(FName StageName)
{
	FEditorScriptExecutionGuard ScriptGuard;
	OnTutorialStageEnded(StageName);
}


void UEditorTutorial::HandleTutorialLaunched()
{
	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.DismissTutorialBrowser();
	FEditorScriptExecutionGuard ScriptGuard;
	OnTutorialLaunched();
}


void UEditorTutorial::HandleTutorialClosed()
{
	FEditorScriptExecutionGuard ScriptGuard;
	OnTutorialClosed();
}


void UEditorTutorial::OpenAsset(UObject* Asset)
{
	if (ensureAsRuntimeWarning(Asset != nullptr))
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
	}
}

AActor* UEditorTutorial::GetActorReference(FString PathToActor)
{
#if WITH_EDITOR
	return Cast<AActor>(StaticFindObject(AActor::StaticClass(), GEditor->GetEditorWorldContext().World(), *PathToActor, EFindObjectFlags::None));
#else
	return nullptr;
#endif //WITH_EDITOR
}

void UEditorTutorial::SetEngineFolderVisibilty(bool bNewVisibility)
{
	bool bDisplayEngine = GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();
	// If we cannot change the vis state, or it matches the new request leave it
	if (bDisplayEngine == bNewVisibility)
	{
		return;
	}
	if (bNewVisibility)
	{
		GetMutableDefault<UContentBrowserSettings>()->SetDisplayEngineFolder(true);
	}
	else
	{
		GetMutableDefault<UContentBrowserSettings>()->SetDisplayEngineFolder(false);
	}
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool UEditorTutorial::GetEngineFolderVisibilty()
{
	return GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();
}
