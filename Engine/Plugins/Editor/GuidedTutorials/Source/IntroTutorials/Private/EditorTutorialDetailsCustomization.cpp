// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTutorialDetailsCustomization.h"
#include "Modules/ModuleManager.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "IIntroTutorials.h"
#include "LevelEditor.h"
#include "EditorTutorial.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "EditorTutorialDetailsCustomization"

TSharedRef<IDetailCustomization> FEditorTutorialDetailsCustomization::MakeInstance()
{
	return MakeShareable( new FEditorTutorialDetailsCustomization );
}

void FEditorTutorialDetailsCustomization::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	struct Local
	{
		static FReply OnLaunchClicked(UEditorTutorial* Tutorial)
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>("LevelEditor");
			IIntroTutorials& IntroTutorials = FModuleManager::Get().GetModuleChecked<IIntroTutorials>("IntroTutorials");
			IntroTutorials.LaunchTutorial(Tutorial, IIntroTutorials::ETutorialStartType::TST_RESTART, LevelEditorModule.GetLevelEditorTab()->GetParentWindow());
			return FReply::Handled();
		}
	};

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);
	check(Objects.Num() > 0);
	UEditorTutorial* Tutorial = CastChecked<UEditorTutorial>(Objects[0].Get());

	IDetailCategoryBuilder& CategoryBuilder = DetailLayout.EditCategory(TEXT("Testing"), LOCTEXT("TestingSection", "Testing"), ECategoryPriority::Important);

	CategoryBuilder.AddCustomRow(LOCTEXT("LaunchButtonLabel", "Launch"))
	.WholeRowContent()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	[
		SNew(SButton)
		.OnClicked(FOnClicked::CreateStatic(&Local::OnLaunchClicked, Tutorial))
		.Text(LOCTEXT("LaunchButtonLabel", "Launch"))
		.ToolTipText(LOCTEXT("LaunchButtonTooltip", "Test this tutorial."))
	];
}


#undef LOCTEXT_NAMESPACE
