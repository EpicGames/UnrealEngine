// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTutorialFactory.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "EditorTutorial.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "AssetTypeCategories.h"

#define LOCTEXT_NAMESPACE "UEditorTutorialFactory"

UEditorTutorialFactory::UEditorTutorialFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UBlueprint::StaticClass();
}

UObject* UEditorTutorialFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FKismetEditorUtilities::CreateBlueprint(UEditorTutorial::StaticClass(), InParent, Name, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), NAME_None);
}

uint32 UEditorTutorialFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

FText UEditorTutorialFactory::GetDisplayName() const
{
	return LOCTEXT("TutorialMenuEntry", "Tutorial Blueprint");
}

#undef LOCTEXT_NAMESPACE
