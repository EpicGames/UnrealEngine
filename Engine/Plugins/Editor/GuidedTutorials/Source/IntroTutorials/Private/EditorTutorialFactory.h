// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "EditorTutorialFactory.generated.h"

UCLASS(hidecategories=Object, collapsecategories)
class UEditorTutorialFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetDisplayName() const override;
	// End of UFactory Interface
};
