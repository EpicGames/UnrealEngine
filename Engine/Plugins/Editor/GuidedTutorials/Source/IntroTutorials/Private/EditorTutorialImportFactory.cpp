// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTutorialImportFactory.h"
#include "Misc/Paths.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "EditorTutorial.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IDocumentationPage.h"
#include "IDocumentation.h"

#define LOCTEXT_NAMESPACE "UEditorTutorialImportFactory"

UEditorTutorialImportFactory::UEditorTutorialImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEditorImport = true;
	bEditAfterNew = true;
	bText = false;
	SupportedClass = UBlueprint::StaticClass();
	Formats.Add(TEXT("udn;UDN documentation files"));
}

bool UEditorTutorialImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString DocDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Documentation/Source"));
	FString NewPath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(Filename));
	FPaths::NormalizeFilename(NewPath);

	return (NewPath.StartsWith(DocDir));
}

static bool GetDocumentationPath(FString& InOutFilePath)
{
	const FString DocDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Documentation/Source"));
	FString NewPath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(InOutFilePath));
	FPaths::NormalizeFilename(NewPath);

	if(NewPath.StartsWith(DocDir))
	{
		InOutFilePath = NewPath.RightChop(DocDir.Len());
		return true;
	}

	return false;
}

UObject* UEditorTutorialImportFactory::FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn)
{
	UBlueprint* NewBlueprint = FKismetEditorUtilities::CreateBlueprint(UEditorTutorial::StaticClass(), InParent, InName, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), NAME_None);
	
	FString PathToImport = GetCurrentFilename();
	if(GetDocumentationPath(PathToImport))
	{
		UEditorTutorial* EditorTutorial = CastChecked<UEditorTutorial>(NewBlueprint->GeneratedClass->GetDefaultObject());
		if(Import(EditorTutorial, PathToImport))
		{
			EditorTutorial->ImportPath = GetCurrentFilename();
		}
	}

	return NewBlueprint;
}

bool UEditorTutorialImportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UBlueprint* Blueprint = Cast<UBlueprint>(Obj);
	if(Blueprint != nullptr && Blueprint->GeneratedClass)
	{
		UEditorTutorial* EditorTutorial = Cast<UEditorTutorial>(Blueprint->GeneratedClass->GetDefaultObject());
		if(EditorTutorial != nullptr)
		{
			OutFilenames.Add(EditorTutorial->ImportPath);
			return true;
		}
	}

	return false;
}

void UEditorTutorialImportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if(NewReimportPaths.Num() > 0)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(Obj);
		if(Blueprint != nullptr)
		{
			UEditorTutorial* EditorTutorial = Cast<UEditorTutorial>(Blueprint->GeneratedClass->GetDefaultObject());
			if(EditorTutorial != nullptr)
			{
				if(FactoryCanImport(NewReimportPaths[0]))
				{
					EditorTutorial->ImportPath = NewReimportPaths[0];
				}
			}
		}
	}
}

EReimportResult::Type UEditorTutorialImportFactory::Reimport(UObject* Obj)
{
	UBlueprint* Blueprint = Cast<UBlueprint>(Obj);
	if(Blueprint != nullptr)
	{
		UEditorTutorial* EditorTutorial = Cast<UEditorTutorial>(Blueprint->GeneratedClass->GetDefaultObject());
		if(EditorTutorial != nullptr)
		{
			FString PathToImport = EditorTutorial->ImportPath;
			if(GetDocumentationPath(PathToImport))
			{
				return Import(EditorTutorial, PathToImport) ? EReimportResult::Succeeded : EReimportResult::Failed;
			}
		}
	}

	return EReimportResult::Failed;
}

int32 UEditorTutorialImportFactory::GetPriority() const
{
	return ImportPriority;
}

bool UEditorTutorialImportFactory::Import(UEditorTutorial* InTutorialToImportTo, const FString& InImportPath)
{
	if(!InImportPath.IsEmpty() && InTutorialToImportTo != nullptr && IDocumentation::Get()->PageExists(InImportPath))
	{
		FDocumentationStyle DocumentationStyle;
		DocumentationStyle
			.ContentStyle(TEXT("Tutorials.Content.Text"))
			.BoldContentStyle(TEXT("Tutorials.Content.TextBold"))
			.NumberedContentStyle(TEXT("Tutorials.Content.Text"))
			.Header1Style(TEXT("Tutorials.Content.HeaderText1"))
			.Header2Style(TEXT("Tutorials.Content.HeaderText2"))
			.HyperlinkStyle(TEXT("Tutorials.Content.Hyperlink"))
			.HyperlinkTextStyle(TEXT("Tutorials.Content.HyperlinkText"))
			.SeparatorStyle(TEXT("Tutorials.Separator"));

		TSharedRef<IDocumentationPage> Page = IDocumentation::Get()->GetPage(InImportPath, TSharedPtr<FParserConfiguration>(), DocumentationStyle);
		InTutorialToImportTo->Modify();
		InTutorialToImportTo->Title = Page->GetTitle();
		InTutorialToImportTo->Stages.Empty();

		TArray<FExcerpt> Excerpts;
		Page->GetExcerpts(Excerpts);
		for(auto& Excerpt : Excerpts)
		{
			Page->GetExcerptContent(Excerpt);

			if(Excerpt.RichText.Len() > 0)
			{
				FTutorialStage& Stage = InTutorialToImportTo->Stages[InTutorialToImportTo->Stages.Add(FTutorialStage())];
				Stage.Name = *Excerpt.Name;
				Stage.Content.Type = ETutorialContent::RichText;

				FString RichText;
				FString* TitleStringPtr = Excerpt.Variables.Find(TEXT("StageTitle"));
				if(TitleStringPtr != nullptr)
				{
					RichText += FString::Printf(TEXT("<TextStyle Style=\"Tutorials.Content.HeaderText2\">%s</>\n\n"), **TitleStringPtr);
				}
				RichText += Excerpt.RichText;

				Stage.Content.Text = FText::FromString(RichText);
			}
		}

		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
