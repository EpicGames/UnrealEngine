// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EditorReimportHandler.h"
#include "Factories/Factory.h"
#include "EditorTutorialImportFactory.generated.h"

class UEditorTutorial;

UCLASS(hidecategories=Object, collapsecategories)
class UEditorTutorialImportFactory : public UFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn) override;
	//~ End UFactory Interface

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler Interface

private:
	/** Import the tutorial from file */
	bool Import(UEditorTutorial* InTutorialToImportTo, const FString& InImportPath);
};
