// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTutorialSettings.h"
#include "Templates/SubclassOf.h"

UEditorTutorialSettings::UEditorTutorialSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UEditorTutorialSettings::FindTutorialInfoForContext(FName InContext, UEditorTutorial*& OutAttractTutorial, UEditorTutorial*& OutLaunchTutorial, FString& OutBrowserFilter) const
{
	for (const auto& Context : TutorialContexts)
	{
		if (Context.Context == InContext)
		{
			OutBrowserFilter = Context.BrowserFilter;

			Context.LaunchTutorial.TryLoad();
			TSubclassOf<UEditorTutorial> LaunchTutorialClass = Context.LaunchTutorial.ResolveClass();
			if (LaunchTutorialClass != nullptr)
			{
				OutLaunchTutorial = LaunchTutorialClass->GetDefaultObject<UEditorTutorial>();
			}

			Context.AttractTutorial.TryLoad();
			TSubclassOf<UEditorTutorial> AttractTutorialClass = Context.AttractTutorial.ResolveClass();
			if (AttractTutorialClass != nullptr)
			{
				OutAttractTutorial = AttractTutorialClass->GetDefaultObject<UEditorTutorial>();
			}

			return;
		}
	}
}
