// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "EditorTutorial.h"
#include "EditorTutorialSettings.generated.h"

/** Named context that corresponds to a particular tutorial */
USTRUCT()
struct FTutorialContext
{
	GENERATED_USTRUCT_BODY()

	/** The context that this tutorial is used in */
	UPROPERTY(EditAnywhere, Category = "Tutorials")
	FName Context;

	/** The filter string to apply to the tutorials browser when launched from this context */
	UPROPERTY(EditAnywhere, Category = "Tutorials")
	FString BrowserFilter;

	/** The tutorial to use in this context to let the user know there is a tutorial available */
	UPROPERTY(EditAnywhere, Category = "Tutorials", meta = (MetaClass = "/Script/IntroTutorials.EditorTutorial"))
	FSoftClassPath AttractTutorial;

	/** The tutorial to use in this context when the user chooses to launch */
	UPROPERTY(EditAnywhere, Category = "Tutorials", meta = (MetaClass = "/Script/IntroTutorials.EditorTutorial"))
	FSoftClassPath LaunchTutorial;
};

/** Editor-wide tutorial settings */
UCLASS(config=GuidedTutorials)
class UEditorTutorialSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Disable the pulsing alert that indicates a new tutorial is available. */
	UPROPERTY(Config, EditAnywhere, Category = "Tutorials")
	bool bDisableAllTutorialAlerts;

	/** Categories for tutorials */
	UPROPERTY(Config, EditAnywhere, Category="Tutorials")
	TArray<FTutorialCategory> Categories;

	/** Tutorial to start on Editor startup */
	UPROPERTY(Config, EditAnywhere, Category="Tutorials", meta=(MetaClass="/Script/IntroTutorials.EditorTutorial"))
	FSoftClassPath StartupTutorial;

	/** Tutorials used in various contexts - e.g. the various asset editors */
	UPROPERTY(Config, EditAnywhere, Category = "Tutorials")
	TArray<FTutorialContext> TutorialContexts;

	/** Get the tutorial info for the specified context */
	void FindTutorialInfoForContext(FName InContext, UEditorTutorial*& OutAttractTutorial, UEditorTutorial*& OutLaunchTutorial, FString& OutBrowserFilter) const;
};
