// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTutorialStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"

FName FEditorTutorialStyle::StyleName("FEditorTutorialStyle");
TUniquePtr<FEditorTutorialStyle> FEditorTutorialStyle::Instance(nullptr);

const FName& FEditorTutorialStyle::GetStyleSetName() const
{
	return StyleName;
}

const FEditorTutorialStyle& FEditorTutorialStyle::Get()
{
	if (!Instance.IsValid())
	{
		Instance = TUniquePtr<FEditorTutorialStyle>(new FEditorTutorialStyle);
	}
	return *(Instance.Get());
}

void FEditorTutorialStyle::Shutdown()
{
	Instance.Reset();
}

FEditorTutorialStyle::FEditorTutorialStyle()
	: FSlateStyleSet(StyleName)
{
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/GuidedTutorials/Content/Editor/Slate"));

	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon128x128(128.0f, 128.0f);

	Set("Tutorials.MenuIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Tutorials", Icon16x16));

	const FLinearColor TutorialButtonColor = FLinearColor(0.15f, 0.15f, 0.15f, 1.0f);
	const FLinearColor TutorialSelectionColor = FLinearColor(0.19f, 0.33f, 0.72f);
	const FLinearColor TutorialNavigationButtonColor = FLinearColor(0.0f, 0.59f, 0.14f, 1.0f);
	const FLinearColor TutorialNavigationButtonHoverColor = FLinearColor(0.2f, 0.79f, 0.34f, 1.0f);
	const FLinearColor TutorialNavigationBackButtonColor = TutorialNavigationButtonColor;
	const FLinearColor TutorialNavigationBackButtonHoverColor = TutorialNavigationButtonHoverColor;

	const FTextBlockStyle TutorialText = FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Documentation.Text"))
		.SetColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(TutorialSelectionColor);

	const FTextBlockStyle TutorialHeaderText = FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Documentation.Header.Text"))
		.SetColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(TutorialSelectionColor);

	Set("Tutorials.Border", new BOX_BRUSH("Tutorials/OverlayFrame", FMargin(18.0f / 64.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)));
	Set("Tutorials.Shadow", new BOX_BRUSH("Tutorials/TutorialShadow", FVector2D(256.0f, 256.0f), FMargin(114.0f / 256.0f)));

	Set("Tutorials.Highlight.Border", new BOX_BRUSH("Tutorials/TutorialBorder", FVector2D(64.0f, 64.0f), FMargin(25.0f / 64.0f)));

	const FTextBlockStyle TutorialBrowserText = FTextBlockStyle(TutorialText)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetHighlightColor(TutorialSelectionColor);

	Set("Tutorials.Browser.Text", TutorialBrowserText);

	Set("Tutorials.Browser.WelcomeHeader", FTextBlockStyle(TutorialBrowserText)
		.SetFontSize(20));

	Set("Tutorials.Browser.SummaryHeader", FTextBlockStyle(TutorialBrowserText)
		.SetFontSize(16));

	Set("Tutorials.Browser.SummaryText", FTextBlockStyle(TutorialBrowserText)
		.SetFontSize(10));

	Set("Tutorials.Browser.HighlightTextColor", TutorialSelectionColor);

	Set("Tutorials.Browser.Button", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0.05f, 0.05f, 0.05f, 1)))
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0.07f, 0.07f, 0.07f, 1)))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0.08f, 0.08f, 0.08f, 1)))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("Tutorials.Browser.BackButton", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 0.0f)))
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 0.05f)))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 0.05f)))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("Tutorials.Content.Button", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0, 0, 0, 0)))
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 1)))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(1, 1, 1, 1)))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("Tutorials.Content.NavigationButtonWrapper", FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("Tutorials.Content.NavigationButton", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), TutorialNavigationButtonColor))
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), TutorialNavigationButtonHoverColor))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), TutorialNavigationButtonHoverColor))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("Tutorials.Content.NavigationBackButton", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), TutorialNavigationBackButtonColor))
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), TutorialNavigationBackButtonHoverColor))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), TutorialNavigationBackButtonHoverColor))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));

	Set("Tutorials.Content.NavigationText", FTextBlockStyle(TutorialText));

	Set("Tutorials.Content.Color", FLinearColor(1.0f, 1.0f, 1.0f, 0.9f));
	Set("Tutorials.Content.Color.Hovered", FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));

	Set("Tutorials.Browser.CategoryArrow", new IMAGE_BRUSH("Tutorials/BrowserCategoryArrow", FVector2D(24.0f, 24.0f), FSlateColor::UseForeground()));
	Set("Tutorials.Browser.DefaultTutorialIcon", new IMAGE_BRUSH("Tutorials/DefaultTutorialIcon_40x", FVector2D(40.0f, 40.0f), FLinearColor::White));
	Set("Tutorials.Browser.DefaultCategoryIcon", new IMAGE_BRUSH("Tutorials/DefaultCategoryIcon_40x", FVector2D(40.0f, 40.0f), FLinearColor::White));

	Set("Tutorials.Browser.BackButton.Image", new IMAGE_BRUSH("Tutorials/BrowserBack", FVector2D(32.0f, 32.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)));
	Set("Tutorials.Browser.PlayButton.Image", new IMAGE_BRUSH("Tutorials/BrowserPlay", FVector2D(32.0f, 32.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)));
	Set("Tutorials.Browser.RestartButton", new IMAGE_BRUSH("Tutorials/BrowserRestart", FVector2D(16.0f, 16.0f), FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)));
	
	Set("Tutorials.Browser.Completed", new IMAGE_BRUSH("Tutorials/TutorialCompleted", Icon32x32));
	Set("Tutorials.Browser.Breadcrumb", new IMAGE_BRUSH("Tutorials/Breadcrumb", Icon8x8, FLinearColor::White));
	Set("Tutorials.Browser.PathText", FTextBlockStyle(TutorialBrowserText)
		.SetFontSize(9));

	Set("Tutorials.Navigation.Button", FButtonStyle()
		.SetNormal(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0, 0, 0, 0)))
		.SetHovered(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0, 0, 0, 0)))
		.SetPressed(CORE_BOX_BRUSH("Common/ButtonHoverHint", FMargin(4 / 16.0f), FLinearColor(0, 0, 0, 0)))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0)));
	
	Set("Tutorials.Navigation.NextButton", new IMAGE_BRUSH("Tutorials/NavigationNext", Icon32x32));
	Set("Tutorials.Navigation.HomeButton", new IMAGE_BRUSH("Tutorials/NavigationHome", Icon32x32));
	Set("Tutorials.Navigation.BackButton", new IMAGE_BRUSH("Tutorials/NavigationBack", Icon32x32));

	Set("Tutorials.WidgetContent", FTextBlockStyle(TutorialText)
		.SetFontSize(10));

	Set("Tutorials.ButtonColor", TutorialButtonColor);
	Set("Tutorials.ButtonHighlightColor", TutorialSelectionColor);
	Set("Tutorials.ButtonDisabledColor", FAppStyle::Get().GetSlateColor("SelectionColor_Inactive"));
	
	Set("Tutorials.ContentAreaBackground", new BOX_BRUSH("Tutorials/TutorialContentBackground", FMargin(4 / 16.0f)));
	Set("Tutorials.HomeContentAreaBackground", new BOX_BRUSH("Tutorials/TutorialHomeContentBackground", FMargin(4 / 16.0f)));
	Set("Tutorials.ContentAreaFrame", new BOX_BRUSH("Tutorials/ContentAreaFrame", FMargin(26.0f / 64.0f)));
	Set("Tutorials.CurrentExcerpt", new IMAGE_BRUSH("Tutorials/CurrentExcerpt", FVector2D(24.0f, 24.0f), TutorialSelectionColor));
	Set("Tutorials.Home", new IMAGE_BRUSH("Tutorials/HomeButton", FVector2D(32.0f, 32.0f)));
	Set("Tutorials.Back", new IMAGE_BRUSH("Tutorials/BackButton", FVector2D(24.0f, 24.0f)));
	Set("Tutorials.Next", new IMAGE_BRUSH("Tutorials/NextButton", FVector2D(24.0f, 24.0f)));
	
	Set("Tutorials.PageHeader", FTextBlockStyle(TutorialHeaderText)
		.SetFontSize(22));

	Set("Tutorials.CurrentExcerpt", FTextBlockStyle(TutorialHeaderText)
		.SetFontSize(16));

	Set("Tutorials.NavigationButtons", FTextBlockStyle(TutorialHeaderText)
		.SetFontSize(16));

	// UDN documentation styles
	Set("Tutorials.Content", FTextBlockStyle(TutorialText)
		.SetColorAndOpacity(FSlateColor::UseForeground()));
	Set("Tutorials.Hyperlink.Text", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Documentation.Hyperlink.Text")));
	Set("Tutorials.NumberedContent", FTextBlockStyle(TutorialText));
	Set("Tutorials.BoldContent", FTextBlockStyle(TutorialText)
		.SetTypefaceFontName(TEXT("Bold")));

	Set("Tutorials.Header1", FTextBlockStyle(TutorialHeaderText)
		.SetFontSize(32));

	Set("Tutorials.Header2", FTextBlockStyle(TutorialHeaderText)
		.SetFontSize(24));
	
	Set("Tutorials.Hyperlink.Button", FButtonStyle(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Documentation.Hyperlink.Button"))
		.SetNormal(CORE_BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor::Black))
		.SetHovered(CORE_BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor::Black)));

	Set("Tutorials.Separator", new CORE_BOX_BRUSH("Common/Separator", 1 / 4.0f, FLinearColor::Black));

	Set("Tutorials.ProgressBar", FProgressBarStyle()
		.SetBackgroundImage(CORE_BOX_BRUSH("Common/ProgressBar_Background", FMargin(5.f / 12.f)))
		.SetFillImage(CORE_BOX_BRUSH("Common/ProgressBar_NeutralFill", FMargin(5.f / 12.f)))
		.SetMarqueeImage(CORE_IMAGE_BRUSH("Common/ProgressBar_Marquee", FVector2D(20, 12), FLinearColor::White, ESlateBrushTileType::Horizontal))
	);

	// Default text styles
	{
		const FTextBlockStyle RichTextNormal = FTextBlockStyle()
			.SetFont(DEFAULT_FONT("Regular", 11))
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetShadowOffset(FVector2D::ZeroVector)
			.SetShadowColorAndOpacity(FLinearColor::Black)
			.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
			.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));
		Set("Tutorials.Content.Text", RichTextNormal);

		Set("Tutorials.Content.TextBold", FTextBlockStyle(RichTextNormal)
			.SetFont(DEFAULT_FONT("Bold", 11)));

		Set("Tutorials.Content.HeaderText1", FTextBlockStyle(RichTextNormal)
			.SetFontSize(20));

		Set("Tutorials.Content.HeaderText2", FTextBlockStyle(RichTextNormal)
			.SetFontSize(16));

		{
			const FButtonStyle RichTextHyperlinkButton = FButtonStyle()
				.SetNormal(CORE_BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor::Blue))
				.SetPressed(FSlateNoResource())
				.SetHovered(CORE_BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor::Blue));

			const FTextBlockStyle RichTextHyperlinkText = FTextBlockStyle(RichTextNormal)
				.SetColorAndOpacity(FLinearColor::Blue);

			Set("Tutorials.Content.HyperlinkText", RichTextHyperlinkText);

			// legacy style
			Set("TutorialEditableText.Editor.HyperlinkText", RichTextHyperlinkText);

			const FHyperlinkStyle RichTextHyperlink = FHyperlinkStyle()
				.SetUnderlineStyle(RichTextHyperlinkButton)
				.SetTextStyle(RichTextHyperlinkText)
				.SetPadding(FMargin(0.0f));
			Set("Tutorials.Content.Hyperlink", RichTextHyperlink);

			Set("Tutorials.Content.ExternalLink", new IMAGE_BRUSH("Tutorials/ExternalLink", Icon16x16, FLinearColor::Blue));

			// legacy style
			Set("TutorialEditableText.Editor.Hyperlink", RichTextHyperlink);
		}
	}

	// Toolbar
	{
		const FLinearColor NormalColor(FColor(0xffeff3f3));
		const FLinearColor SelectedColor(FColor(0xffdbe4d5));
		const FLinearColor HoverColor(FColor(0xffdbe4e4));
		const FLinearColor DisabledColor(FColor(0xaaaaaa));
		const FLinearColor TextColor(FColor(0xff2c3e50));

		Set("TutorialEditableText.RoundedBackground", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(FColor(0xffeff3f3))));

		Set("TutorialEditableText.Toolbar.HyperlinkImage", new IMAGE_BRUSH("Tutorials/hyperlink", Icon16x16, TextColor));
		Set("TutorialEditableText.Toolbar.ImageImage", new IMAGE_BRUSH("Tutorials/Image", Icon16x16, TextColor));

		Set("TutorialEditableText.Toolbar.TextColor", TextColor);

		Set("TutorialEditableText.Toolbar.Text", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(TextColor)
		);

		Set("TutorialEditableText.Toolbar.BoldText", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(TextColor)
		);

		Set("TutorialEditableText.Toolbar.ItalicText", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.SetFont(DEFAULT_FONT("Italic", 10))
			.SetColorAndOpacity(TextColor)
		);

		Set("TutorialEditableText.Toolbar.Checkbox", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
			.SetUncheckedImage(IMAGE_BRUSH("Common/CheckBox", Icon16x16, FLinearColor::White))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Common/CheckBox", Icon16x16, HoverColor))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Common/CheckBox_Hovered", Icon16x16, HoverColor))
			.SetCheckedImage(IMAGE_BRUSH("Common/CheckBox_Checked_Hovered", Icon16x16, FLinearColor::White))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Common/CheckBox_Checked_Hovered", Icon16x16, HoverColor))
			.SetCheckedPressedImage(IMAGE_BRUSH("Common/CheckBox_Checked", Icon16x16, HoverColor))
			.SetUndeterminedImage(IMAGE_BRUSH("Common/CheckBox_Undetermined", Icon16x16, FLinearColor::White))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH("Common/CheckBox_Undetermined_Hovered", Icon16x16, HoverColor))
			.SetUndeterminedPressedImage(IMAGE_BRUSH("Common/CheckBox_Undetermined_Hovered", Icon16x16, FLinearColor::White))
		);

		Set("TutorialEditableText.Toolbar.ToggleButtonCheckbox", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), NormalColor))
			.SetUncheckedHoveredImage(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor))
			.SetUncheckedPressedImage(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor))
			.SetCheckedImage(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), SelectedColor))
			.SetCheckedHoveredImage(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor))
			.SetCheckedPressedImage(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor))
		);

		const FButtonStyle TutorialButton = FButtonStyle()
			.SetNormal(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), NormalColor))
			.SetHovered(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor))
			.SetPressed(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), SelectedColor))
			.SetNormalPadding(FMargin(2, 2, 2, 2))
			.SetPressedPadding(FMargin(2, 3, 2, 1));
		Set("TutorialEditableText.Toolbar.Button", TutorialButton);

		const FComboButtonStyle ComboButton = FComboButtonStyle()
			.SetButtonStyle(FButtonStyle())
			.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
			.SetMenuBorderBrush(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), NormalColor))
			.SetMenuBorderPadding(FMargin(0.0f));
		Set("TutorialEditableText.Toolbar.ComboButton", ComboButton);

		{
			const FButtonStyle ComboBoxButton = FButtonStyle()
				.SetNormal(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White))
				.SetHovered(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White))
				.SetPressed(BOX_BRUSH("Tutorials/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White))
				.SetNormalPadding(FMargin(2, 2, 2, 2))
				.SetPressedPadding(FMargin(2, 3, 2, 1));

			const FComboButtonStyle ComboBoxComboButton = FComboButtonStyle(ComboButton)
				.SetButtonStyle(ComboBoxButton)
				.SetMenuBorderPadding(FMargin(1.0));

			Set("TutorialEditableText.Toolbar.ComboBox", FComboBoxStyle()
				.SetComboButtonStyle(ComboBoxComboButton)
			);
		}
	}

	Set("TutorialLaunch.Button", FButtonStyle()
		.SetNormalPadding(0)
		.SetPressedPadding(0)
		.SetNormal(IMAGE_BRUSH("Tutorials/TutorialButton_Default_16x", Icon16x16))
		.SetHovered(IMAGE_BRUSH("Tutorials/TutorialButton_Hovered_16x", Icon16x16))
		.SetPressed(IMAGE_BRUSH("Tutorials/TutorialButton_Pressed_16x", Icon16x16))
	);

	Set("TutorialLaunch.Circle", new IMAGE_BRUSH("Tutorials/Circle_128x", Icon128x128, FLinearColor::White));
	Set("TutorialLaunch.Circle.Color", FLinearColor::Green);

	Set("ClassIcon.EditorTutorial", new IMAGE_BRUSH_SVG("AssetIcons/EditorTutorial_16", Icon16x16));
	Set("ClassThumbnail.EditorTutorial", new IMAGE_BRUSH_SVG("AssetIcons/EditorTutorial_64", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FEditorTutorialStyle::~FEditorTutorialStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
