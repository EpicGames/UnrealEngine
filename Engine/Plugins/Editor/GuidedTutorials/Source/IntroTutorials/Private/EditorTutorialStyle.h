// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FEditorTutorialStyle :
    public FSlateStyleSet
{
public:
	virtual const FName& GetStyleSetName() const override;

	static const FEditorTutorialStyle& Get();
	static void Shutdown();

	~FEditorTutorialStyle();

private:
	FEditorTutorialStyle();

	static FName StyleName;
	static TUniquePtr<FEditorTutorialStyle> Instance;
};

