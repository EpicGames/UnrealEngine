// Copyright Epic Games, Inc. All Rights Reserved.

#include "IntroTutorials.h"
#include "Templates/SubclassOf.h"
#include "Curves/CurveFloat.h"
#include "Misc/CommandLine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "PropertyEditorModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "BlueprintEditorModule.h"
#include "GameProjectGenerationModule.h"
#include "SourceCodeNavigation.h"

#include "EditorTutorial.h"
#include "EditorTutorialSettings.h"
#include "EditorTutorialStyle.h"
#include "TutorialStateSettings.h"
#include "TutorialSettings.h"
#include "ISettingsModule.h"
#include "STutorialsBrowser.h"
#include "TutorialStructCustomization.h"
#include "EditorTutorialDetailsCustomization.h"
#include "STutorialRoot.h"
#include "STutorialLoading.h"
#include "STutorialButton.h"
#include "Toolkits/ToolkitManager.h"
#include "BlueprintEditor.h"
#include "Misc/EngineBuildSettings.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "Widgets/Docking/SDockTab.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "IClassTypeActions.h"
#include "ClassTypeActions_EditorTutorial.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "IntroTutorials"

IMPLEMENT_MODULE(FIntroTutorials, IntroTutorials)


FIntroTutorials::FIntroTutorials()
	: CurrentObjectClass(nullptr)
	, ContentIntroCurve(nullptr)
{
	bDisableTutorials = true;

	bDesireResettingTutorialSeenFlagOnLoad = FParse::Param(FCommandLine::Get(), TEXT("ResetTutorials"));
}

FString FIntroTutorials::AnalyticsEventNameFromTutorial(UEditorTutorial* Tutorial)
{
	FString TutorialPath = Tutorial->GetOutermost()->GetFName().ToString();

	// strip off everything but the asset name
	FString RightStr;
	TutorialPath.Split( TEXT("/"), NULL, &RightStr, ESearchCase::IgnoreCase, ESearchDir::FromEnd );

	return RightStr;
}

TSharedRef<FExtender> FIntroTutorials::AddSummonBlueprintTutorialsMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> EditingObjects) const
{
	UObject* PrimaryObject = nullptr;
	if(EditingObjects.Num() > 0)
	{
		PrimaryObject = EditingObjects[0];
	}

	TSharedRef<FExtender> Extender(new FExtender());

	Extender->AddMenuExtension(
		"HelpBrowse",
		EExtensionHook::After,
		CommandList,
		FMenuExtensionDelegate::CreateRaw(const_cast<FIntroTutorials*>(this), &FIntroTutorials::AddSummonBlueprintTutorialsMenuExtension, PrimaryObject));

	return Extender;
}

void FIntroTutorials::StartupModule()
{
	FEditorTutorialStyle::Get();

	// This code can run with content commandlets. Slate is not initialized with commandlets and the below code will fail.
	const bool bCommandlet = IsRunningCommandlet();
	const bool bUnattended = FApp::IsUnattended();
	const bool bCanEverRender = FApp::CanEverRender();

	if (!bDisableTutorials && !bCommandlet && !bUnattended && bCanEverRender)
	{
		// Add tutorial for main frame opening
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().AddRaw(this, &FIntroTutorials::MainFrameLoad);
		MainFrameModule.OnMainFrameSDKNotInstalled().AddRaw(this, &FIntroTutorials::HandleSDKNotInstalled);
		MainFrameModule.OnMainFrameRequestResource().AddRaw(this, &FIntroTutorials::HandleRequestResource);

		// Add menu option for level editor tutorial
		RegisterSummonTutorialsMenuEntries();

		// Add menu option to blueprint editor as well
		FBlueprintEditorModule& BPEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );
		BPEditorModule.GetMenuExtensibilityManager()->GetExtenderDelegates().Add(FAssetEditorExtender::CreateRaw(this, &FIntroTutorials::AddSummonBlueprintTutorialsMenuExtender));

		// Add hook for when AddToCodeProject dialog window is opened
		FGameProjectGenerationModule::Get().OnAddCodeToProjectDialogOpened().AddRaw(this, &FIntroTutorials::OnAddCodeToProjectDialogOpened);

		// Add hook for New Project dialog window is opened
		//FGameProjectGenerationModule::Get().OnNewProjectProjectDialogOpened().AddRaw(this, &FIntroTutorials::OnNewProjectDialogOpened);

		FSourceCodeNavigation::AccessOnCompilerNotFound().AddRaw( this, &FIntroTutorials::HandleCompilerNotFound );
	
		// maybe reset all the "have I seen this once" flags
		if (bDesireResettingTutorialSeenFlagOnLoad)
		{
			GetMutableDefault<UTutorialStateSettings>()->ClearProgress();
		}

		// register our class actions to show the "Play" button on editor tutorial Blueprint assets
		{
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			TSharedRef<IClassTypeActions> EditorTutorialClassActions = MakeShareable(new FClassTypeActions_EditorTutorial());
			RegisteredClassTypeActions.Add(EditorTutorialClassActions);
			AssetTools.RegisterClassTypeActions(EditorTutorialClassActions);
		}
	}

	// Register to display our settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Editor", "General", "Tutorials",
			LOCTEXT("EditorTutorialSettingsName", "Tutorials"),
			LOCTEXT("EditorTutorialSettingsDescription", "Control what tutorials are available in the Editor."),
			GetMutableDefault<UEditorTutorialSettings>()
		);

		SettingsModule->RegisterSettings("Project", "Engine", "Tutorials",
			LOCTEXT("TutorialSettingsName", "Tutorials"),
			LOCTEXT("TutorialSettingsDescription", "Control what tutorials are available in this project."),
			GetMutableDefault<UTutorialSettings>()
		);
	}

	// register details customizations
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TutorialContent", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTutorialContentCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TutorialContentAnchor", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTutorialContentAnchorCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomClassLayout("EditorTutorial", FOnGetDetailCustomizationInstance::CreateStatic(&FEditorTutorialDetailsCustomization::MakeInstance));

	UCurveFloat* ContentIntroCurveAsset = LoadObject<UCurveFloat>(nullptr, TEXT("/Engine/Tutorial/ContentIntroCurve.ContentIntroCurve"));
	if(ContentIntroCurveAsset)
	{
		ContentIntroCurveAsset->AddToRoot();
		ContentIntroCurve = ContentIntroCurveAsset;
	}

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TEXT("TutorialsBrowser"), FOnSpawnTab::CreateRaw(this, &FIntroTutorials::SpawnTutorialsBrowserTab))
		.SetMenuType(ETabSpawnerMenuType::Hidden);
}

void FIntroTutorials::ShutdownModule()
{
	UToolMenus::UnregisterOwner(this);

	if (!bDisableTutorials && !IsRunningCommandlet())
	{
		FSourceCodeNavigation::AccessOnCompilerNotFound().RemoveAll( this );

		FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools");
		if (AssetToolsModule)
		{
			IAssetTools& AssetTools = AssetToolsModule->Get();
			for(const auto& RegisteredClassTypeAction : RegisteredClassTypeActions)
			{
				AssetTools.UnregisterClassTypeActions(RegisteredClassTypeAction);
			}
		}
	}

	if (BlueprintEditorExtender.IsValid() && FModuleManager::Get().IsModuleLoaded("Kismet"))
	{
		FBlueprintEditorModule& BPEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		BPEditorModule.GetMenuExtensibilityManager()->RemoveExtender(BlueprintEditorExtender);
	}
	if (MainMenuExtender.IsValid() && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
		LevelEditorModule.GetMenuExtensibilityManager()->RemoveExtender(MainMenuExtender);
	}
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		MainFrameModule.OnMainFrameCreationFinished().RemoveAll(this);
	}

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Editor", "General", "Tutorials");
		SettingsModule->UnregisterSettings("Project", "Engine", "Tutorials");
	}

	if(FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TutorialContent");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TutorialWidgetReference");
		PropertyEditorModule.UnregisterCustomClassLayout("EditorTutorial");
	}

	if(ContentIntroCurve.IsValid())
	{
		ContentIntroCurve.Get()->RemoveFromRoot();
		ContentIntroCurve = nullptr;
	}

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TEXT("TutorialsBrowser"));

	FEditorTutorialStyle::Shutdown();
}

void FIntroTutorials::RegisterSummonTutorialsMenuEntries()
{
	FToolMenuOwnerScoped OwnerScoped(this);
	UToolMenu* HelpMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Help");
	FToolMenuSection& Section = HelpMenu->AddSection("Tutorials", LOCTEXT("TutorialsLabel", "Tutorials"), FToolMenuInsert("HelpBrowse", EToolMenuInsertType::After));
	Section.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Tutorials",
		LOCTEXT("TutorialsMenuEntryTitle", "Tutorials"),
		LOCTEXT("TutorialsMenuEntryToolTip", "Opens up introductory tutorials covering the basics of using the Unreal Engine Editor."),
		FSlateIcon(FEditorTutorialStyle::Get().GetStyleSetName(), "Tutorials.MenuIcon"),
		FUIAction(FExecuteAction::CreateRaw(this, &FIntroTutorials::SummonTutorialHome))
	));
}

void FIntroTutorials::AddSummonBlueprintTutorialsMenuExtension(FMenuBuilder& MenuBuilder, UObject* PrimaryObject)
{
	MenuBuilder.BeginSection("Tutorials", LOCTEXT("TutorialsLabel", "Tutorials"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("BlueprintMenuEntryTitle", "Blueprint Overview"),
		LOCTEXT("BlueprintMenuEntryToolTip", "Opens up an introductory overview of Blueprints."),
		FSlateIcon(FEditorTutorialStyle::Get().GetStyleSetName(), "Tutorials.MenuIcon"),
		FUIAction(FExecuteAction::CreateRaw(this, &FIntroTutorials::SummonBlueprintTutorialHome, PrimaryObject, true)));

	if(PrimaryObject != nullptr)
	{
		UBlueprint* BP = Cast<UBlueprint>(PrimaryObject);
		if(BP != nullptr)
		{
			UEnum* Enum = StaticEnum<EBlueprintType>();
			check(Enum);
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("BlueprintTutorialsMenuEntryTitle", "{0} Tutorial"), Enum->GetDisplayNameTextByValue(BP->BlueprintType)),
				LOCTEXT("BlueprintTutorialsMenuEntryToolTip", "Opens up an introductory tutorial covering this particular part of the Blueprint editor."),
				FSlateIcon(FEditorTutorialStyle::Get().GetStyleSetName(), "Tutorials.MenuIcon"),
				FUIAction(FExecuteAction::CreateRaw(this, &FIntroTutorials::SummonBlueprintTutorialHome, PrimaryObject, false)));
		}
	}

	MenuBuilder.EndSection();
}

void FIntroTutorials::MainFrameLoad(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog)
{
	if (!bIsRunningStartupDialog)
	{
		// install a root widget for the tutorial overlays to hang off
		if(InRootWindow.IsValid() && !TutorialRoot.IsValid())
		{
			InRootWindow->AddOverlaySlot()
				[
					SAssignNew(TutorialRoot, STutorialRoot)
				];
		}

		// See if we should show 'welcome' screen
		MaybeOpenWelcomeTutorial();
	}
}

void FIntroTutorials::SummonTutorialHome()
{
	SummonTutorialBrowser();
}

void FIntroTutorials::SummonBlueprintTutorialHome(UObject* Asset, bool bForceWelcome)
{
	UBlueprint* BP = CastChecked<UBlueprint>(Asset);

	FName Context("BlueprintOverview");
	if(!bForceWelcome)
	{
		Context = FBlueprintEditor::GetContextFromBlueprintType(BP->BlueprintType);
	}

	UEditorTutorial* AttractTutorial = nullptr;
	UEditorTutorial* LaunchTutorial = nullptr;
	FString BrowserFilter;
	GetDefault<UEditorTutorialSettings>()->FindTutorialInfoForContext(Context, AttractTutorial, LaunchTutorial, BrowserFilter);

	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));

	if (LaunchTutorial != nullptr)
	{
		TSharedPtr<SWindow> ContextWindow;
		TSharedPtr< IToolkit > Toolkit = FToolkitManager::Get().FindEditorForAsset(Asset);
		if(ensure(Toolkit.IsValid()))
		{
			ContextWindow = FSlateApplication::Get().FindWidgetWindow(Toolkit->GetToolkitHost()->GetParentWidget());
			check(ContextWindow.IsValid());
		}

		IntroTutorials.LaunchTutorial(LaunchTutorial, IIntroTutorials::ETutorialStartType::TST_RESTART, ContextWindow);
	}
}

bool FIntroTutorials::MaybeOpenWelcomeTutorial()
{
	if(FParse::Param(FCommandLine::Get(), TEXT("TestTutorialAlerts")) || !FEngineBuildSettings::IsInternalBuild())
	{
		// try editor startup tutorial
		TSubclassOf<UEditorTutorial> EditorStartupTutorialClass = LoadClass<UEditorTutorial>(NULL, *GetDefault<UEditorTutorialSettings>()->StartupTutorial.ToString(), NULL, LOAD_None, NULL);
		if(EditorStartupTutorialClass != nullptr)
		{
			UEditorTutorial* Tutorial = EditorStartupTutorialClass->GetDefaultObject<UEditorTutorial>();
			if (!GetDefault<UTutorialStateSettings>()->HaveSeenTutorial(Tutorial))
			{
				LaunchTutorial(Tutorial);
				return true;
			}
		}

		// Try project startup tutorial
		const FString ProjectStartupTutorialPathStr = GetDefault<UTutorialSettings>()->StartupTutorial.ToString();
		if (!ProjectStartupTutorialPathStr.IsEmpty())
		{
			TSubclassOf<UEditorTutorial> ProjectStartupTutorialClass = LoadClass<UEditorTutorial>(NULL, *ProjectStartupTutorialPathStr, NULL, LOAD_None, NULL);
			if (ProjectStartupTutorialClass != nullptr)
			{
				UEditorTutorial* Tutorial = ProjectStartupTutorialClass->GetDefaultObject<UEditorTutorial>();
				if (!GetDefault<UTutorialStateSettings>()->HaveSeenTutorial(Tutorial))
				{
					LaunchTutorial(Tutorial);
					return true;
				}
			}
		}
	}

	return false;
}

void FIntroTutorials::OnAddCodeToProjectDialogOpened()
{
	// @todo: add code to project dialog tutorial here?
//	MaybeOpenWelcomeTutorial(AddCodeToProjectWelcomeTutorial);
}

void FIntroTutorials::OnNewProjectDialogOpened()
{
	// @todo: new project dialog tutorial here?
//	MaybeOpenWelcomeTutorial(TemplateOverview);
}

void FIntroTutorials::HandleCompilerNotFound()
{
#if PLATFORM_WINDOWS
	LaunchTutorialByName( TEXT( "/Engine/Tutorial/Installation/InstallingVisualStudioTutorial.InstallingVisualStudioTutorial" ) );
#elif PLATFORM_MAC
	LaunchTutorialByName( TEXT( "/Engine/Tutorial/Installation/InstallingXCodeTutorial.InstallingXCodeTutorial" ) );
#else
	STUBBED("FIntroTutorials::HandleCompilerNotFound");
#endif
}

void FIntroTutorials::HandleSDKNotInstalled(const FString& PlatformName, const FString& InTutorialAsset)
{
	if (FPackageName::IsValidLongPackageName(InTutorialAsset, true))
	{
		UBlueprint* Blueprint = LoadObject<UBlueprint>(nullptr, *InTutorialAsset);
		if (Blueprint)
		{
			LaunchTutorialByName(InTutorialAsset);
		}
	}
}

void FIntroTutorials::HandleRequestResource(const FString& Category, const FString& InTutorialAsset)
{
	if (Category == "Tutorial")
	{
		if (FPackageName::IsValidLongPackageName(InTutorialAsset, true))
		{
			UBlueprint* Blueprint = LoadObject<UBlueprint>(nullptr, *InTutorialAsset);
			if (Blueprint)
			{
				LaunchTutorialByName(InTutorialAsset);
			}
		}
	}
}

EVisibility FIntroTutorials::GetHomeButtonVisibility() const
{
	if(CurrentObjectClass != nullptr)
	{
		return CurrentObjectClass->IsChildOf(UBlueprint::StaticClass()) ? EVisibility::Hidden : EVisibility::Visible;
	}
	
	return EVisibility::Visible;
}

FOnIsPicking& FIntroTutorials::OnIsPicking()
{
	return OnIsPickingDelegate;
}

FOnValidatePickingCandidate& FIntroTutorials::OnValidatePickingCandidate()
{
	return OnValidatePickingCandidateDelegate;
}

void FIntroTutorials::SummonTutorialBrowser()
{
	if(TutorialRoot.IsValid())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>( TEXT("LevelEditor") );
		TutorialBrowserDockTab = LevelEditorModule.GetLevelEditorTabManager()->TryInvokeTab(FTabId("TutorialsBrowser"));
	}
}

void FIntroTutorials::DismissTutorialBrowser()
{
	if (TutorialBrowserDockTab.IsValid() && TutorialBrowserDockTab.Pin().IsValid())
	{
		TutorialBrowserDockTab.Pin()->RequestCloseTab();
		TutorialBrowserDockTab = nullptr;
	}
}

void FIntroTutorials::AttachWidget(TSharedPtr<SWidget> Widget)
{
	if (TutorialRoot.IsValid())
	{
		TutorialRoot->AttachWidget(Widget);
	}
}

void FIntroTutorials::DetachWidget()
{
	if (TutorialRoot.IsValid())
	{
		TutorialRoot->DetachWidget();
	}
}

void FIntroTutorials::LaunchTutorial(const FString& TutorialAssetName, TSharedPtr<SWindow> InNavigationWindow)
{
	LaunchTutorialByName(TutorialAssetName, true, InNavigationWindow);
}

void FIntroTutorials::LaunchTutorialByName(const FString& InAssetPath, bool bInRestart, TWeakPtr<SWindow> InNavigationWindow, FSimpleDelegate OnTutorialClosed, FSimpleDelegate OnTutorialExited)
{
	UBlueprint* Blueprint = LoadObject<UBlueprint>(nullptr, *InAssetPath);
	if (Blueprint && Blueprint->GeneratedClass)
	{
		UEditorTutorial* TutorialObject = NewObject<UEditorTutorial>(GetTransientPackage(), Blueprint->GeneratedClass);
		LaunchTutorial(TutorialObject, bInRestart ? IIntroTutorials::ETutorialStartType::TST_RESTART : IIntroTutorials::ETutorialStartType::TST_CONTINUE, InNavigationWindow, OnTutorialClosed, OnTutorialExited);
	}
}

void FIntroTutorials::LaunchTutorial(UEditorTutorial* InTutorial, ETutorialStartType InStartType, TWeakPtr<SWindow> InNavigationWindow, FSimpleDelegate OnTutorialClosed, FSimpleDelegate OnTutorialExited)
{
	if(TutorialRoot.IsValid())
	{
		if(!InNavigationWindow.IsValid())
		{
			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			InNavigationWindow = MainFrameModule.GetParentWindow();
		}
		// TODO We will want to protect against this on rewrite.
		// check(!InTutorial->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject));
		TutorialRoot->LaunchTutorial(InTutorial, InStartType, InNavigationWindow, OnTutorialClosed, OnTutorialExited);
	}
}

void FIntroTutorials::CloseAllTutorialContent()
{
	if (TutorialRoot.IsValid())
	{
		TutorialRoot->CloseAllTutorialContent();
	}
}

void FIntroTutorials::GoToPreviousStage()
{
	if (TutorialRoot.IsValid())
	{
		TutorialRoot->GoToPreviousStage();
	}
}

void FIntroTutorials::GoToNextStage(TWeakPtr<SWindow> InNavigationWindow)
{
	if (TutorialRoot.IsValid())
	{
		TutorialRoot->GoToNextStage(InNavigationWindow);
	}
}

TSharedRef<SWidget> FIntroTutorials::CreateTutorialsWidget(FName InContext, TWeakPtr<SWindow> InContextWindow) const
{
	if (FGlobalTabmanager::Get()->GetTabPermissionList()->PassesFilter(TEXT("TutorialsBrowser")))
	{
		return SNew(STutorialButton)
				.Context(InContext)
				.ContextWindow(InContextWindow);
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

TSharedPtr<SWidget> FIntroTutorials::CreateTutorialsLoadingWidget(TWeakPtr<SWindow> InContextWindow) const
{
	return SNew(STutorialLoading)
		.ContextWindow(InContextWindow);
}

float FIntroTutorials::GetIntroCurveValue(float InTime)
{
	if(ContentIntroCurve.IsValid())
	{
		return ContentIntroCurve.Get()->GetFloatValue(InTime);
	}

	return 1.0f;
}

TSharedRef<SDockTab> FIntroTutorials::SpawnTutorialsBrowserTab(const FSpawnTabArgs& SpawnTabArgs)
{
	TAttribute<FText> Label = LOCTEXT("TutorialsBrowserTabLabel", "Tutorials");

	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(Label)
		.ToolTip(IDocumentation::Get()->CreateToolTip(Label, nullptr, "Shared/TutorialsBrowser", "Tab"));	

	TSharedRef<STutorialsBrowser> TutorialsBrowser = SNew(STutorialsBrowser)
		.ExternalCategories(ExternalCategories)
		.OnLaunchTutorial(FOnLaunchTutorial::CreateRaw(this, &FIntroTutorials::LaunchTutorial));

	NewTab->SetContent(TutorialsBrowser);

	return NewTab;
}

void FIntroTutorials::RegisterCategory(FTutorialCategory NewCategory)
{
	ExternalCategories.Add(NewCategory);
}



#undef LOCTEXT_NAMESPACE
