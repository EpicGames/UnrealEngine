// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "IIntroTutorials.h"
#include "Widgets/SWindow.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

class FMenuBuilder;
class FSpawnTabArgs;
class IClassTypeActions;
class STutorialRoot;
class UCurveFloat;
class UEditorTutorial;

/** Query delegate to see if we are in picking mode */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsPicking, FName& /* OutWidgetNameToHighlight */);

/** Query delegate for name of any picked widget and check if it matches the 'pickable' name of the given widget */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnValidatePickingCandidate, TSharedRef<SWidget> /*InWidget*/, FName& /* OutWidgetNameToHighlight */, bool& /*bOutShouldHighlight*/);

class UEditorTutorial;
struct FTutorialContent;
class STutorialRoot;
class IClassTypeActions;

class FIntroTutorials : public IIntroTutorials
{
public:
	// ctor
	FIntroTutorials();

	/** Get the delegate used to check for whether we are picking widgets */
	FOnIsPicking& OnIsPicking();
	
	/** Get the delegate used to validate the given widget for pickings */
	FOnValidatePickingCandidate& OnValidatePickingCandidate();

	void GoToPreviousStage();

	void GoToNextStage(TWeakPtr<SWindow> InNavigationWindow);

	float GetIntroCurveValue(float InTime);

	void SummonTutorialBrowser();

	void DismissTutorialBrowser();

	void AttachWidget(TSharedPtr<SWidget> Widget);

	void DetachWidget();

	static FString AnalyticsEventNameFromTutorial(UEditorTutorial* Tutorial);

private:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Register the menu for summoning the tutorial */
	void RegisterSummonTutorialsMenuEntries();

	/** Add a menu extender to summon context-sensitive Blueprints page */
	TSharedRef<FExtender> AddSummonBlueprintTutorialsMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> EditingObjects) const;

	/** Add a menu entry to summon context-sensitive Blueprints page */
	void AddSummonBlueprintTutorialsMenuExtension(FMenuBuilder& MenuBuilder, UObject* PrimaryObject);

	/** Event to be called when the main frame is loaded */
	void MainFrameLoad(TSharedPtr<SWindow> InRootWindow, bool bIsRunningStartupDialog);

	/** Summon tutorial home page to front */
	void SummonTutorialHome();

	/** Summon blueprint tutorial home page to front */
	void SummonBlueprintTutorialHome(UObject* Asset, bool bForceWelcome);

	/** Event to be called when Tutorial window is closed. */
	void OnTutorialWindowClosed(const TSharedRef<SWindow>& Window);

	/** Called when tutorial is dismissed, either when finished or aborted. */
	void OnTutorialDismissed() const;

	/** Event to be called when any asset editor is opened */
	void OnAssetEditorOpened(UObject* Asset);

	/** Events to call when editor changes state in various ways */
	void OnAddCodeToProjectDialogOpened();
	void OnNewProjectDialogOpened();

	/** Events to call when opening the compiler fails */
	void HandleCompilerNotFound();

	/** Events to call when SDK isn't installed */
	void HandleSDKNotInstalled(const FString& PlatformName, const FString& InTutorialAsset);

	/** Handles calls to MainFrameRequestResource that specify a tutorial asset and "Tutorial" category. */
	void HandleRequestResource(const FString& Category, const FString& InTutorialName);

	bool MaybeOpenWelcomeTutorial();
	void ResetWelcomeTutorials() const;

	/** Delegate for home button visibility */
	EVisibility GetHomeButtonVisibility() const;

	/** Internal helper to launch a tutorial from a path */
	void LaunchTutorialByName(const FString& InAssetPath, bool bRestart = true, TWeakPtr<SWindow> InNavigationWindow = nullptr, FSimpleDelegate OnTutorialClosed = FSimpleDelegate(), FSimpleDelegate OnTutorialExited = FSimpleDelegate());

	/** Spawn a tab for browsing tutorials */
	TSharedRef<SDockTab> SpawnTutorialsBrowserTab(const FSpawnTabArgs& SpawnTabArgs);

	/** When we create a tutorial browser tab, we only keep a weak pointer to it. This way, it can be closed/destroyed without our involvement, but we can also close/destroy it ourselves. */
	TWeakPtr<SDockTab> TutorialBrowserDockTab;

public:

	// IIntroTutorials interface
	virtual void LaunchTutorial(const FString& TutorialAssetName, TSharedPtr<SWindow> InNavigationWindow = nullptr) override;
	virtual void LaunchTutorial(UEditorTutorial* Tutorial, IIntroTutorials::ETutorialStartType InStartType = IIntroTutorials::ETutorialStartType::TST_RESTART, TWeakPtr<SWindow> InNavigationWindow = nullptr, FSimpleDelegate OnTutorialClosed = FSimpleDelegate(), FSimpleDelegate OnTutorialExited = FSimpleDelegate()) override;
	virtual void CloseAllTutorialContent() override;
	virtual TSharedRef<SWidget> CreateTutorialsWidget(FName InContext, TWeakPtr<SWindow> InContextWindow = nullptr) const override;
	virtual TSharedPtr<SWidget> CreateTutorialsLoadingWidget(TWeakPtr<SWindow> InContextWindow = nullptr) const override;
	virtual void RegisterCategory(FTutorialCategory NewCategory) override;
	// End of IIntroTutorials interface
private:
	/** The tab id of the tutorial tab */
	static const FName IntroTutorialTabName;

	/** The extender to pass to the level editor to extend it's window menu */
	TSharedPtr<FExtender> MainMenuExtender;

	/** The extender to pass to the blueprint editor to extend it's window menu */
	TSharedPtr<FExtender> BlueprintEditorExtender;

	/** Whether tutorials are disabled altogether */
	bool bDisableTutorials;

	/** The current object we are using as a basis for displaying a tutorial */
	UClass* CurrentObjectClass;

	/** should we be clearing the 'have seen this tutorial' flag? (controlled by -tutorials on the command line) */
	bool bDesireResettingTutorialSeenFlagOnLoad;

	/** Delegate used to determine whether we are in picking mode */
	FOnIsPicking OnIsPickingDelegate;
	
	/** Delegate used to determine if we are in picking mode, get the name of any picked widget and check if it matches the 'pickable' name of the given widget */
	FOnValidatePickingCandidate OnValidatePickingCandidateDelegate;

	/** Root widget for tutorial overlay system */
	TSharedPtr<STutorialRoot> TutorialRoot;

	/** Curve asset for intros */
	TWeakObjectPtr<UCurveFloat> ContentIntroCurve;

	/** The collection of registered class type actions. */
	TArray<TSharedRef<IClassTypeActions>> RegisteredClassTypeActions;

	/** Extra tutorial categories that have been defined externally */
	TArray<FTutorialCategory> ExternalCategories;
};
