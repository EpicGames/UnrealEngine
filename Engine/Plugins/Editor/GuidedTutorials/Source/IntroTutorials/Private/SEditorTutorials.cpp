// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditorTutorials.h"
#include "IntroTutorials.h"
#include "EditorTutorial.h"

#define LOCTEXT_NAMESPACE "TutorialsBrowser"

void SEditorTutorials::Construct(const FArguments& InArgs)
{
	bIsNavigationWindow = false;
	ParentWindow = InArgs._ParentWindow;
	OnNextClicked = InArgs._OnNextClicked;
	OnBackClicked = InArgs._OnBackClicked;
	OnHomeClicked = InArgs._OnHomeClicked;
	OnCloseClicked = InArgs._OnCloseClicked;
	OnGetCurrentTutorial = InArgs._OnGetCurrentTutorial;
	OnGetCurrentTutorialStage = InArgs._OnGetCurrentTutorialStage;
	OnWidgetWasDrawn = InArgs._OnWidgetWasDrawn;
	OnWasWidgetDrawn = InArgs._OnWasWidgetDrawn;

	ContentBox = SNew(SHorizontalBox);

	ChildSlot
	[
		ContentBox.ToSharedRef()
	];

	RebuildCurrentContent();
}

void SEditorTutorials::LaunchTutorial(bool bInIsNavigationWindow, FSimpleDelegate InOnTutorialClosed, FSimpleDelegate InOnTutorialExited)
{
	bIsNavigationWindow = bInIsNavigationWindow;
	OnTutorialClosed = InOnTutorialClosed;
	OnTutorialExited = InOnTutorialExited;

	RebuildCurrentContent();
}

void SEditorTutorials::HideContent()
{
	HandleHomeClicked();
	bIsNavigationWindow = false;

	RebuildCurrentContent();
}

bool SEditorTutorials::IsNavigationVisible() const
{
	return bIsNavigationWindow;
}

EVisibility SEditorTutorials::GetBrowserVisibility() const
{
	return OnGetCurrentTutorial.Execute() == nullptr ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SEditorTutorials::GetNavigationVisibility() const
{
	return EVisibility::Collapsed;
}

void SEditorTutorials::HandleCloseClicked()
{
	OnCloseClicked.ExecuteIfBound();
	OnTutorialClosed.ExecuteIfBound();

	OverlayContent.Reset();
	ContentBox->ClearChildren();
	OnTutorialExited.ExecuteIfBound();
}

void SEditorTutorials::HandleBackClicked()
{
	// forward to other overlays so they can rebuild their widgets as well
	OnBackClicked.ExecuteIfBound();
}

void SEditorTutorials::HandleHomeClicked()
{
	OnHomeClicked.ExecuteIfBound();
	ContentBox->ClearChildren();
	OnTutorialExited.ExecuteIfBound();

	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.SummonTutorialBrowser();
}

void SEditorTutorials::HandleNextClicked()
{
	// forward to other overlays so they can rebuild their widgets as well
	OnNextClicked.ExecuteIfBound(ParentWindow);
}

bool SEditorTutorials::IsBackButtonEnabled() const
{
	UEditorTutorial* CurrentTutorial = OnGetCurrentTutorial.Execute();
	int32 CurrentTutorialStage = OnGetCurrentTutorialStage.Execute();

	const bool bStepsPassed = (CurrentTutorial != nullptr) && (CurrentTutorialStage > 0);
	const bool bPreviousTutorialAvailable = (CurrentTutorial != nullptr) && (CurrentTutorialStage == 0) && (CurrentTutorial->PreviousTutorial != nullptr);
	return bStepsPassed || bPreviousTutorialAvailable;
}

bool SEditorTutorials::IsHomeButtonEnabled() const
{
	return true;
}

bool SEditorTutorials::IsNextButtonEnabled() const
{
	UEditorTutorial* CurrentTutorial = OnGetCurrentTutorial.Execute();
	int32 CurrentTutorialStage = OnGetCurrentTutorialStage.Execute();

	const bool bStepsRemaining = (CurrentTutorial != nullptr) && (CurrentTutorialStage + 1 < CurrentTutorial->Stages.Num());
	const bool bNextTutorialAvailable = (CurrentTutorial != nullptr) && (CurrentTutorialStage + 1 >= CurrentTutorial->Stages.Num()) && (CurrentTutorial->NextTutorial != nullptr);
	return bStepsRemaining || bNextTutorialAvailable;
}

float SEditorTutorials::GetProgress() const
{
	UEditorTutorial* CurrentTutorial = OnGetCurrentTutorial.Execute();
	int32 CurrentTutorialStage = OnGetCurrentTutorialStage.Execute();

	return (CurrentTutorial != nullptr && CurrentTutorial->Stages.Num() > 0) ? (float)(CurrentTutorialStage + 1) / (float)CurrentTutorial->Stages.Num() : 0.0f;
}

void SEditorTutorials::RebuildCurrentContent()
{
	UEditorTutorial* CurrentTutorial = OnGetCurrentTutorial.Execute();
	int32 CurrentTutorialStage = OnGetCurrentTutorialStage.Execute();

	OverlayContent = nullptr;
	ContentBox->ClearChildren();
	if(CurrentTutorial != nullptr && CurrentTutorialStage < CurrentTutorial->Stages.Num())
	{
		ContentBox->AddSlot()
		[
			SAssignNew(OverlayContent, STutorialOverlay, CurrentTutorial, &CurrentTutorial->Stages[CurrentTutorialStage])
			.OnClosed(FSimpleDelegate::CreateSP(this, &SEditorTutorials::HandleCloseClicked))
			.IsStandalone(CurrentTutorial->bIsStandalone)
			.ParentWindow(ParentWindow)
			.AllowNonWidgetContent(bIsNavigationWindow)
			.OnBackClicked(FSimpleDelegate::CreateSP(this, &SEditorTutorials::HandleBackClicked))
			.OnHomeClicked(FSimpleDelegate::CreateSP(this, &SEditorTutorials::HandleHomeClicked))
			.OnNextClicked(FSimpleDelegate::CreateSP(this, &SEditorTutorials::HandleNextClicked))
			.IsBackEnabled(this, &SEditorTutorials::IsBackButtonEnabled)
			.IsHomeEnabled(this, &SEditorTutorials::IsHomeButtonEnabled)
			.IsNextEnabled(this, &SEditorTutorials::IsNextButtonEnabled)
			.OnWidgetWasDrawn(OnWidgetWasDrawn)
			.OnWasWidgetDrawn(OnWasWidgetDrawn)
		];
	}
	else
	{
		// create 'empty' overlay, as we may need this for picking visualization
		ContentBox->AddSlot()
		[
			SAssignNew(OverlayContent, STutorialOverlay, nullptr, nullptr)
			.OnClosed(FSimpleDelegate::CreateSP(this, &SEditorTutorials::HandleCloseClicked))
			.IsStandalone(false)
			.ParentWindow(ParentWindow)
			.AllowNonWidgetContent(false)
		];	
	}
}

TSharedPtr<SWindow> SEditorTutorials::GetParentWindow() const
{
	return ParentWindow.IsValid() ? ParentWindow.Pin() : TSharedPtr<SWindow>();
}

#undef LOCTEXT_NAMESPACE
