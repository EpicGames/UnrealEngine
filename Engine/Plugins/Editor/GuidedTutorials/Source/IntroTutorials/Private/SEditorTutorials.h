// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "STutorialsBrowser.h"
#include "STutorialOverlay.h"

class SWindow;
class UEditorTutorial;

/** Delegate fired when next button is clicked */
DECLARE_DELEGATE_OneParam(FOnNextClicked, TWeakPtr<SWindow> /*InNavigationWindow*/);

/** Delegate fired to retrieve current tutorial */
DECLARE_DELEGATE_RetVal(UEditorTutorial*, FOnGetCurrentTutorial);

/** Delegate fired to retrieve current tutorial stage */
DECLARE_DELEGATE_RetVal(int32, FOnGetCurrentTutorialStage);

/**
 * Container widget for all tutorial-related widgets
 */
class SEditorTutorials : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SEditorTutorials )
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

	SLATE_EVENT(FSimpleDelegate, OnCloseClicked)
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
	SLATE_EVENT(FOnNextClicked, OnNextClicked)
	SLATE_EVENT(FSimpleDelegate, OnBackClicked)
	SLATE_EVENT(FSimpleDelegate, OnHomeClicked)
	SLATE_EVENT(FOnGetCurrentTutorial, OnGetCurrentTutorial)
	SLATE_EVENT(FOnGetCurrentTutorialStage, OnGetCurrentTutorialStage)
	SLATE_EVENT(FOnLaunchTutorial, OnLaunchTutorial)
	SLATE_EVENT(FOnWidgetWasDrawn, OnWidgetWasDrawn)
	SLATE_EVENT(FOnWasWidgetDrawn, OnWasWidgetDrawn)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Launch a tutorial - will interrogate parent to get the tutorial data to display */
	void LaunchTutorial(bool bInIsNavigationWindow, FSimpleDelegate InOnTutorialClosed, FSimpleDelegate InOnTutorialExited);

	/** hide the tutorials browser in this window */
	void HideContent();

	/** Get parent window for this widget */
	TSharedPtr<SWindow> GetParentWindow() const;

	/** Rebuild content according to the current tutorial state */
	void RebuildCurrentContent();
	
	/** @return whether the navigation controls are currently visible */
	bool IsNavigationVisible() const;

private:
	/** Handle whether we should display the browser */
	EVisibility GetBrowserVisibility() const;

	/** Handle whether we should display the navigation buttons */
	EVisibility GetNavigationVisibility() const;

	/** Handle hitting the close button on the browser or a standalone widget */
	void HandleCloseClicked();

	/** Handle hitting the home button in the navigation window */
	void HandleHomeClicked();

	/** Handle hitting the back button in the navigation window */
	void HandleBackClicked();

	/** Handle hitting the next button in the navigation window */
	void HandleNextClicked();

	/** Get enabled state of the back button */
	bool IsBackButtonEnabled() const;

	/** Get enabled state of the home button */
	bool IsHomeButtonEnabled() const;

	/** Get enabled state of the next button */
	bool IsNextButtonEnabled() const;

	/** Get the current progress to display in the progress bar */
	float GetProgress() const;

private:
	/** Box that contains varied content for current tutorial */
	TSharedPtr<SHorizontalBox> ContentBox;

	/** Content widget for current tutorial */
	TSharedPtr<STutorialOverlay> OverlayContent;

	/** Window that contains this widget */
	TWeakPtr<SWindow> ParentWindow;

	/** Whether the browser is visible */
	bool bBrowserVisible;

	/** Whether we should display navigation */
	bool bIsNavigationWindow;

	/** Delegate fired when next button is clicked */
	FOnNextClicked OnNextClicked;

	/** Delegate fired when back button is clicked */
	FSimpleDelegate OnBackClicked;

	/** Delegate fired when home button is clicked */
	FSimpleDelegate OnHomeClicked;

	/** Delegate fired when close button is clicked */
	FSimpleDelegate OnCloseClicked;

	/** Delegate fired to retrive the current tutorial */
	FOnGetCurrentTutorial OnGetCurrentTutorial;

	/** Delegate fired to retrive the current tutorial stage */
	FOnGetCurrentTutorialStage OnGetCurrentTutorialStage;

	/** External delegates used to report user interaction */
	FSimpleDelegate OnTutorialClosed;
	FSimpleDelegate OnTutorialExited;

	/** Delegates for registering & querying whether a widget was drawn */
	FOnWidgetWasDrawn OnWidgetWasDrawn;
	FOnWasWidgetDrawn OnWasWidgetDrawn;
};
