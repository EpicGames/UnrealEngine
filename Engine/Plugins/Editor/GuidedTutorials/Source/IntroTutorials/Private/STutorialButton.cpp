// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialButton.h"
#include "Rendering/DrawElements.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "IIntroTutorials.h"
#include "IntroTutorials.h"
#include "EditorTutorialSettings.h"
#include "EditorTutorialStyle.h"
#include "TutorialStateSettings.h"
#include "Misc/EngineBuildSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"

#define LOCTEXT_NAMESPACE "STutorialButton"

namespace TutorialButtonConstants
{
	const float MaxPulseOffset = 32.0f;
	const float PulseAnimationLength = 2.0f;
}

void STutorialButton::Construct(const FArguments& InArgs)
{
	Context = InArgs._Context;
	ContextWindow = InArgs._ContextWindow;

	bTestAlerts = FParse::Param(FCommandLine::Get(), TEXT("TestTutorialAlerts"));

	bPendingClickAction = false;
	bTutorialAvailable = false;
	bTutorialCompleted = false;
	bTutorialDismissed = false;
	AlertStartTime = 0.0f;

	PulseAnimation.AddCurve(0.0f, TutorialButtonConstants::PulseAnimationLength, ECurveEaseFunction::Linear);
	RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &STutorialButton::OpenTutorialPostConstruct ) );

	IIntroTutorials& IntroTutorials = FModuleManager::LoadModuleChecked<IIntroTutorials>(TEXT("IntroTutorials"));
	LoadingWidget = IntroTutorials.CreateTutorialsLoadingWidget(ContextWindow);

	ChildSlot
	[
		SNew(SButton)
		.AddMetaData<FTagMetaData>(*FString::Printf(TEXT("%s.TutorialLaunchButton"), *Context.ToString()))
		.ButtonStyle(FEditorTutorialStyle::Get(), "TutorialLaunch.Button")
		.ToolTipText(this, &STutorialButton::GetButtonToolTip)
		.OnClicked(this, &STutorialButton::HandleButtonClicked)
		.ContentPadding(0.0f)
		[
			SNew(SBox)
			.WidthOverride(16)
			.HeightOverride(16)
		]
	];
}

EActiveTimerReturnType STutorialButton::OpenTutorialPostConstruct( double InCurrentTime, float InDeltaTime )
{
	// Begin playing the pulse animation on a loop
	PulseAnimation.Play(this->AsShared(), true);

	RefreshStatus();

	if (bTutorialAvailable && CachedAttractTutorial != nullptr && !bTutorialDismissed && !bTutorialCompleted && !GetMutableDefault<UTutorialStateSettings>()->AreAllTutorialsDismissed())
	{
		// kick off the attract tutorial if the user hasn't dismissed it and hasn't completed it
		FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>( TEXT( "IntroTutorials" ) );
		const bool bRestart = true;
		IntroTutorials.LaunchTutorial(CachedAttractTutorial, bRestart ? IIntroTutorials::ETutorialStartType::TST_RESTART : IIntroTutorials::ETutorialStartType::TST_CONTINUE, ContextWindow);
	}

	if ( ShouldShowAlert() )
	{
		AlertStartTime = FPlatformTime::Seconds();
	}

	if ( CachedLaunchTutorial != nullptr )
	{
		TutorialTitle = CachedLaunchTutorial->Title;
	}

	return EActiveTimerReturnType::Stop;
}

static void GetAnimationValues(float InAnimationProgress, float& OutAlphaFactor0, float& OutPulseFactor0, float& OutAlphaFactor1, float& OutPulseFactor1)
{
	InAnimationProgress = FMath::Fmod(InAnimationProgress * 2.0f, 1.0f);

	OutAlphaFactor0 = FMath::Square(1.0f - InAnimationProgress);
	OutPulseFactor0 = 1.0f - FMath::Square(1.0f - InAnimationProgress);

	float OffsetProgress = FMath::Fmod(InAnimationProgress + 0.25f, 1.0f);
	OutAlphaFactor1 = FMath::Square(1.0f - OffsetProgress);
	OutPulseFactor1 = 1.0f - FMath::Square(1.0f - OffsetProgress);
}

int32 STutorialButton::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled) + 1000;

	if ( PulseAnimation.IsPlaying() )
	{
		float AlphaFactor0 = 0.0f;
		float AlphaFactor1 = 0.0f;
		float PulseFactor0 = 0.0f;
		float PulseFactor1 = 0.0f;
		GetAnimationValues(PulseAnimation.GetLerp(), AlphaFactor0, PulseFactor0, AlphaFactor1, PulseFactor1);

		const FSlateBrush* PulseBrush = FEditorTutorialStyle::Get().GetBrush(TEXT("TutorialLaunch.Circle"));
		const FLinearColor PulseColor = FEditorTutorialStyle::Get().GetColor(TEXT("TutorialLaunch.Circle.Color"));

		// We should be clipped by the window size, not our containing widget, as we want to draw outside the widget
		const FVector2D WindowSize = OutDrawElements.GetPaintWindow()->GetSizeInScreen();
		FSlateRect WindowClippingRect(0.0f, 0.0f, WindowSize.X, WindowSize.Y);

		{
			FVector2D PulseOffset = FVector2D(PulseFactor0 * TutorialButtonConstants::MaxPulseOffset, PulseFactor0 * TutorialButtonConstants::MaxPulseOffset);

			FVector2D BorderPosition = (FVector2D(AllottedGeometry.AbsolutePosition) - ((FVector2D(PulseBrush->Margin.Left, PulseBrush->Margin.Top) * PulseBrush->ImageSize * AllottedGeometry.Scale) + PulseOffset));
			FVector2D BorderSize = ((AllottedGeometry.GetLocalSize() * AllottedGeometry.Scale) + (PulseOffset * 2.0f) + (FVector2D(PulseBrush->Margin.Right * 2.0f, PulseBrush->Margin.Bottom * 2.0f) * PulseBrush->ImageSize * AllottedGeometry.Scale));

			FPaintGeometry BorderGeometry(BorderPosition, BorderSize, AllottedGeometry.Scale);

			// draw highlight border
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, BorderGeometry, PulseBrush, ESlateDrawEffect::None, FLinearColor(PulseColor.R, PulseColor.G, PulseColor.B, AlphaFactor0));
		}

		{
			FVector2D PulseOffset = FVector2D(PulseFactor1 * TutorialButtonConstants::MaxPulseOffset, PulseFactor1 * TutorialButtonConstants::MaxPulseOffset);

			FVector2D BorderPosition = (FVector2D(AllottedGeometry.AbsolutePosition) - ((FVector2D(PulseBrush->Margin.Left, PulseBrush->Margin.Top) * PulseBrush->ImageSize * AllottedGeometry.Scale) + PulseOffset));
			FVector2D BorderSize = ((AllottedGeometry.Size * AllottedGeometry.Scale) + (PulseOffset * 2.0f) + (FVector2D(PulseBrush->Margin.Right * 2.0f, PulseBrush->Margin.Bottom * 2.0f) * PulseBrush->ImageSize * AllottedGeometry.Scale));

			FPaintGeometry BorderGeometry(BorderPosition, BorderSize, AllottedGeometry.Scale);

			// draw highlight border
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, BorderGeometry, PulseBrush, ESlateDrawEffect::None, FLinearColor(PulseColor.R, PulseColor.G, PulseColor.B, AlphaFactor1));
		}
	}

	return LayerId;
}

FReply STutorialButton::HandleButtonClicked()
{
	if (bPendingClickAction)
	{
		//There's already a click pending
		return FReply::Handled();
	}

	RefreshStatus();

	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Context"), Context.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TimeSinceAlertStarted"), (AlertStartTime != 0.0f && ShouldShowAlert()) ? (FPlatformTime::Seconds() - AlertStartTime) : -1.0f));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LaunchedBrowser"), ShouldLaunchBrowser()));

		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Rocket.Tutorials.ClickedContextButton"), EventAttributes);
	}

	bPendingClickAction = true;
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &STutorialButton::HandleButtonClicked_AssetRegistryChecker));
	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.AttachWidget(LoadingWidget);
	return FReply::Handled();
}

EActiveTimerReturnType STutorialButton::HandleButtonClicked_AssetRegistryChecker(double InCurrentTime, float InDeltaTime)
{
	//Force tutorials to load into the asset registry before we proceed any further.
	FAssetRegistryModule& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	bool IsStillLoading = AssetRegistry.Get().IsLoadingAssets();
	if (IsStillLoading)
	{
		//We could tick the asset registry here, but we don't need to.
		return EActiveTimerReturnType::Continue;
	}

	//Sometimes, this gives a false positive because the tutorial we want to launch wasn't loaded into the asset registry when we checked. Opening and closing the tab works around that by letting the browser recheck.
	if (ShouldLaunchBrowser())
	{
		RefreshStatus();
	}

	//Now we know the asset registry is loaded, the tutorial broswer is updated, and we are ready to complete the click and stop this active timer
	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.DetachWidget();
	if (ShouldLaunchBrowser())
	{
		IntroTutorials.SummonTutorialBrowser();
	}
	else if (CachedLaunchTutorial != nullptr)
	{
		//If we don't want to launch the browser, and we have a tutorial in mind, launch the tutorial now.
		auto Delegate = FSimpleDelegate::CreateSP(this, &STutorialButton::HandleTutorialExited);

		IntroTutorials.LaunchTutorial(CachedLaunchTutorial, IIntroTutorials::ETutorialStartType::TST_RESTART, ContextWindow, Delegate, Delegate);

		// The user asked to start the tutorial, so we don't need to remind them about it again.
		// We used to remind them in future sessions, but user preference is that we don't.
		const bool bDismissAcrossSessions = true;
		GetMutableDefault<UTutorialStateSettings>()->DismissTutorial(CachedLaunchTutorial, bDismissAcrossSessions);
		GetMutableDefault<UTutorialStateSettings>()->SaveProgress();
		bTutorialDismissed = true;
	}

	bPendingClickAction = false;
	return EActiveTimerReturnType::Stop;
}

FReply STutorialButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

		const bool bShouldShowAlert = ShouldShowAlert();
		const bool bAreAllTutorialsDismissed = GetMutableDefault<UTutorialStateSettings>()->AreAllTutorialsDismissed();

		if (bShouldShowAlert || !bAreAllTutorialsDismissed)
		{
			if (bShouldShowAlert)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("DismissReminder", "Don't Remind Me Again"),
					LOCTEXT("DismissReminderTooltip", "Selecting this option will prevent the tutorial blip from being displayed again, even if you choose not to complete the tutorial."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &STutorialButton::DismissAlert))
					);
			}

			if (!bAreAllTutorialsDismissed)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("DismissAllReminders", "Dismiss All Tutorial Reminders"),
					LOCTEXT("DismissAllRemindersTooltip", "Selecting this option will prevent all tutorial blips from being displayed."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &STutorialButton::DismissAllAlerts))
					);
			}

			MenuBuilder.AddMenuSeparator();
		}

		if(bTutorialAvailable)
		{
			MenuBuilder.AddMenuEntry(
				FText::Format(LOCTEXT("LaunchTutorialPattern", "Start Tutorial: {0}"), TutorialTitle),
				FText::Format(LOCTEXT("TutorialLaunchToolTip", "Click to begin the '{0}' tutorial"), TutorialTitle),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &STutorialButton::LaunchTutorial))
				);
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("LaunchBrowser", "Show Available Tutorials"),
			LOCTEXT("LaunchBrowserTooltip", "Display the tutorials browser"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &STutorialButton::LaunchBrowser))
			);

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
	}
	return FReply::Handled();
}

void STutorialButton::DismissAlert()
{
	RefreshStatus();

	if( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Context"), Context.ToString()));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TimeSinceAlertStarted"), (AlertStartTime != 0.0f && ShouldShowAlert()) ? (FPlatformTime::Seconds() - AlertStartTime) : -1.0f));

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.DismissedTutorialAlert"), EventAttributes );
	}

	// If they actually right click and choose "Dismiss Alert", we'll go ahead and suppress the tutorial reminder for this feature for good (all sessions.)
	const bool bDismissAcrossSessions = true;
	if (CachedAttractTutorial != nullptr)
	{
		GetMutableDefault<UTutorialStateSettings>()->DismissTutorial(CachedAttractTutorial, bDismissAcrossSessions);
		CachedAttractTutorial = nullptr;
	}
	if( CachedLaunchTutorial != nullptr)
	{
		GetMutableDefault<UTutorialStateSettings>()->DismissTutorial(CachedLaunchTutorial, bDismissAcrossSessions);
		CachedLaunchTutorial = nullptr;
	}
	GetMutableDefault<UTutorialStateSettings>()->SaveProgress();
	bTutorialDismissed = true;

	RefreshStatus();
}

void STutorialButton::DismissAllAlerts()
{
	GetMutableDefault<UTutorialStateSettings>()->DismissAllTutorials();
	DismissAlert();		//TODO FIXME Call this for all STutorialButtons that are currently displayed so they all stop pulsing.
}

void STutorialButton::LaunchTutorial()
{
	HandleButtonClicked();
}

void STutorialButton::LaunchBrowser()
{
	RefreshStatus();

	FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
	IntroTutorials.SummonTutorialBrowser();
}

bool STutorialButton::ShouldLaunchBrowser() const
{
	return (!bTutorialAvailable || bTutorialCompleted || bTutorialDismissed);
}

bool STutorialButton::ShouldShowAlert() const
{
	if (!GIsDemoMode && (bTestAlerts || !FEngineBuildSettings::IsInternalBuild()) && bTutorialAvailable && !(bTutorialCompleted || bTutorialDismissed))
	{
		return (!GetMutableDefault<UEditorTutorialSettings>()->bDisableAllTutorialAlerts && !GetMutableDefault<UTutorialStateSettings>()->AreAllTutorialsDismissed());
	}
	return false;
}

FText STutorialButton::GetButtonToolTip() const
{
	if(ShouldLaunchBrowser())
	{
		return LOCTEXT("TutorialLaunchBrowserToolTip", "Show Available Tutorials...");
	}
	else if(bTutorialAvailable)
	{
		return FText::Format(LOCTEXT("TutorialLaunchToolTipPattern", "Click to begin the '{0}' tutorial, or right click for more options"), TutorialTitle);
	}
	
	return LOCTEXT("TutorialToolTip", "Take Tutorial");
}

void STutorialButton::RefreshStatus()
{
	CachedAttractTutorial = nullptr;
	CachedLaunchTutorial = nullptr;
	CachedBrowserFilter = TEXT("");
	GetDefault<UEditorTutorialSettings>()->FindTutorialInfoForContext(Context, CachedAttractTutorial, CachedLaunchTutorial, CachedBrowserFilter);

	bTutorialAvailable = (CachedLaunchTutorial != nullptr);
	bTutorialCompleted = (CachedLaunchTutorial != nullptr) && GetDefault<UTutorialStateSettings>()->HaveCompletedTutorial(CachedLaunchTutorial);
	bTutorialDismissed = ((CachedAttractTutorial != nullptr) && GetDefault<UTutorialStateSettings>()->IsTutorialDismissed(CachedAttractTutorial)) ||
						 ((CachedLaunchTutorial != nullptr) && GetDefault<UTutorialStateSettings>()->IsTutorialDismissed(CachedLaunchTutorial));

	if (ShouldShowAlert())
	{
		PulseAnimation.Resume();
	}
	else
	{
		PulseAnimation.Pause();
	}
}

void STutorialButton::HandleTutorialExited()
{
	RefreshStatus();
}

#undef LOCTEXT_NAMESPACE
