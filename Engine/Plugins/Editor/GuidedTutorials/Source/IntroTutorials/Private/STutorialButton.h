// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Animation/CurveSequence.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class SWindow;
class UEditorTutorial;

class STutorialButton : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STutorialButton) {}
	
	SLATE_ARGUMENT(FName, Context)

	SLATE_ARGUMENT(TWeakPtr<SWindow>, ContextWindow)

	SLATE_END_ARGS()

	/** Widget constructor */
	void Construct(const FArguments& Args);

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	
private:
	/** Opens the tutorial post-construct */
	EActiveTimerReturnType OpenTutorialPostConstruct( double InCurrentTime, float InDeltaTime );

	/** Handle clicking the tutorial button */
	FReply HandleButtonClicked();

	/** After the initial click is processed, go here, wait for the asset registry to load, and complete the action once we're ready */
	EActiveTimerReturnType HandleButtonClicked_AssetRegistryChecker(double InCurrentTime, float InDeltaTime);

	/** Dismiss the pulsing alert */
	void DismissAlert();

	/** Dismiss all pulsing alerts */
	void DismissAllAlerts();

	/** Launch the tutorials browser */
	void LaunchBrowser();

	/** Check whether we should launch the browser in this context */
	bool ShouldLaunchBrowser() const;

	/** Check whether we should show the alert in this context */
	bool ShouldShowAlert() const;

	/** Get the tooltip for the tutorials button */
	FText GetButtonToolTip() const;

	/** Launch tutorial from the context menu */
	void LaunchTutorial();

	/** Refresh internal status of flags, tutorials, filters etc.*/
	void RefreshStatus();

	/** Handle tutorial exiting/finishing */
	void HandleTutorialExited();

private:
	
	/** Whether we have a tutorial for this context */
	bool bTutorialAvailable;

	/** Whether we have completed the tutorial for this content */
	bool bTutorialCompleted;

	/** Whether we have dismissed the tutorial for this content */
	bool bTutorialDismissed;

	/** Flag to force alerts to appear in internal builds (caches command line -TestTutorialAlerts) */
	bool bTestAlerts;

	/** Context that this widget was created for (i.e. what part of the editor) */
	FName Context;

	/** Window that the tutorial should be launched in */
	TWeakPtr<SWindow> ContextWindow;

	/** Animation curve for displaying pulse */
	FCurveSequence PulseAnimation;

	/** Start time we began playing the alert animation */
	float AlertStartTime;

	/**	The name of the tutorial we will launch */
	FText TutorialTitle;

	/** Cached attract tutorial */
	UEditorTutorial* CachedAttractTutorial;

	/** Cached launch tutorial */
	UEditorTutorial* CachedLaunchTutorial;

	/** Cached browser filter */
	FString CachedBrowserFilter;

	TSharedPtr<SWidget> LoadingWidget;

	/** True if we're waiting for asset registry to load in response to a click */
	bool bPendingClickAction;
};
