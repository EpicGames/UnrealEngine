// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialContent.h"
#include "EditorTutorialStyle.h"
#include "Rendering/DrawElements.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "IIntroTutorials.h"
#include "IntroTutorials.h"
#include "IDocumentationPage.h"
#include "IDocumentation.h"
#include "TutorialText.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "STutorialContent"

namespace TutorialConstants
{
	const float BorderPulseAnimationLength = 0.75f;
	const float BorderIntroAnimationLength = 0.4f;
	const float ContentIntroAnimationLength = 0.25f;
	const float MinBorderOpacity = 0.1f;
	const float ShadowScale = 8.0f;
	const float MaxBorderOffset = 8.0f;
	const FMargin BorderSizeStandalone(24.0f, 24.0f);
	const FMargin BorderSize(24.0f, 24.0f, 24.0f, 62.0f);
}

const float ContentOffset = 10.0f;

void STutorialContent::Construct(const FArguments& InArgs, UEditorTutorial* InTutorial, const FTutorialContent& InContent)
{
	bIsVisible = Anchor.Type == ETutorialAnchorIdentifier::None;

	Tutorial = InTutorial;
	
	VerticalAlignment = InArgs._VAlign;
	HorizontalAlignment = InArgs._HAlign;
	WidgetOffset = InArgs._Offset;
	bIsStandalone = InArgs._IsStandalone;
	OnClosed = InArgs._OnClosed;
	OnNextClicked = InArgs._OnNextClicked;
	OnHomeClicked = InArgs._OnHomeClicked;
	OnBackClicked = InArgs._OnBackClicked;
	IsBackEnabled = InArgs._IsBackEnabled;
	IsHomeEnabled = InArgs._IsHomeEnabled;
	IsNextEnabled = InArgs._IsNextEnabled;
	Anchor = InArgs._Anchor;
	bAllowNonWidgetContent = InArgs._AllowNonWidgetContent;
	OnWasWidgetDrawn = InArgs._OnWasWidgetDrawn;
	NextButtonText = InArgs._NextButtonText;
	BackButtonText = InArgs._BackButtonText;

	BorderIntroAnimation.AddCurve(0.0f, TutorialConstants::BorderIntroAnimationLength, ECurveEaseFunction::CubicOut);
	BorderPulseAnimation.AddCurve(0.0f, TutorialConstants::BorderPulseAnimationLength, ECurveEaseFunction::Linear);
	BorderIntroAnimation.Play(this->AsShared());
	
	// Set the border pulse to play on a loop and immediately pause it - will be resumed when needed
	BorderPulseAnimation.Play(this->AsShared(), true);
	BorderPulseAnimation.Pause();

	ContentIntroAnimation.AddCurve(0.0f, TutorialConstants::ContentIntroAnimationLength, ECurveEaseFunction::Linear);
	ContentIntroAnimation.Play(this->AsShared());

	if (InContent.Text.IsEmpty() == true)
	{
		ChildSlot
		[
			SAssignNew(ContentWidget, SBorder)
			.Visibility(EVisibility::SelfHitTestInvisible)
		];
		return;
	}

	ChildSlot
	[
		SNew(SFxWidget)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.RenderScale(this, &STutorialContent::GetAnimatedZoom)
		.RenderScaleOrigin(FVector2D(0.5f, 0.5f))
		[
			SNew(SOverlay)
			.Visibility(this, &STutorialContent::GetVisibility)
			+SOverlay::Slot()
			[
				SAssignNew(ContentWidget, SBorder)

				// Add more padding if the content is to be displayed centrally (i.e. not on a widget)
				.Padding(bIsStandalone ? TutorialConstants::BorderSizeStandalone : TutorialConstants::BorderSize)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.BorderImage(FEditorTutorialStyle::Get().GetBrush("Tutorials.Border"))
				.BorderBackgroundColor(this, &STutorialContent::GetBackgroundColor)
				.ForegroundColor(FAppStyle::Get().GetSlateColor("InvertedForeground"))
				[
					SNew(SFxWidget)
					.RenderScale(this, &STutorialContent::GetInverseAnimatedZoom)
					.RenderScaleOrigin(FVector2D(0.5f, 0.5f))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.MaxWidth(600.0f)
							.VAlign(VAlign_Center)
							[
								GenerateContentWidget(InContent, DocumentationPage, TAttribute<FText>(), false, InArgs._WrapTextAt)
							]
						]
					]
				]
			]
			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Right)
			.Padding(16.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				.Padding(2.0f)
				[
					SNew(SComboButton)
					.ToolTipText(LOCTEXT("MoreOptionsTooltip", "More Options"))
					.Visibility(this, &STutorialContent::GetMenuButtonVisibility)
					.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.Button"))
					.ContentPadding(0.0f)
					.OnGetMenuContent(FOnGetContent::CreateSP(this, &STutorialContent::HandleGetMenuContent))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				.Padding(0.0f)
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("QuitStandaloneTooltip", "Close this Message"))
					.OnClicked(this, &STutorialContent::OnCloseButtonClicked)
					.Visibility(this, &STutorialContent::GetCloseButtonVisibility)
					.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.Button"))
					.ContentPadding(0.0f)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Symbols.X"))
						.ColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f))
					]
				]
			]
			+ SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Left)
			.Padding(12.0f)
			[
				SAssignNew(BackButton, SButton)
				.ToolTipText(this, &STutorialContent::GetBackButtonTooltip)
				.OnClicked(this, &STutorialContent::HandleBackButtonClicked)
				.Visibility(this, &STutorialContent::GetBackButtonVisibility)
				.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.NavigationButtonWrapper"))
				.ContentPadding(0.0f)
				[
					SNew(SBox)
					.Padding(8.0f)
					[
						SNew(SBorder)
						.BorderImage(this, &STutorialContent::GetBackButtonBorder)
						[
 							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SImage)
									.Image(this, &STutorialContent::GetBackButtonBrush)
									.ColorAndOpacity(FLinearColor::White)
								]
						]
					]
				]
			] 
			+ SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			.Padding(12.0f)
			[
				SAssignNew(NextButton, SButton)
				.ToolTipText(this, &STutorialContent::GetNextButtonTooltip)
				.OnClicked(this, &STutorialContent::HandleNextClicked)
				.Visibility(this, &STutorialContent::GetMenuButtonVisibility)
				.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.NavigationButtonWrapper"))
				.ContentPadding(0.0f)
				[
					SNew(SBox)
					.Padding(8.0f)
					[
						SNew(SBorder)
						.BorderImage(this, &STutorialContent::GetNextButtonBorder)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(4.0f, 0.0f, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.Text(this, &STutorialContent::GetNextButtonLabel)
								.TextStyle(FEditorTutorialStyle::Get(), "Tutorials.Content.NavigationText")
								.ColorAndOpacity(FLinearColor::White)
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SImage)
								.Image(this, &STutorialContent::GetNextButtonBrush)
								.ColorAndOpacity(FLinearColor::White)
							]
						]
					]
				]
			]
		]
	];
}

//static void GetAnimationValues(bool bIsIntro, float InAnimationProgress, float& OutAlphaFactor, float& OutPulseFactor, FLinearColor& OutShadowTint, FLinearColor& OutBorderTint)
//{
//	if ( bIsIntro )
//	{
//		OutAlphaFactor = InAnimationProgress;
//		OutPulseFactor = ( 1.0f - OutAlphaFactor ) * 50.0f;
//		OutShadowTint = FLinearColor(1.0f, 1.0f, 0.0f, OutAlphaFactor);
//		OutBorderTint = FLinearColor(1.0f, 1.0f, 0.0f, OutAlphaFactor * OutAlphaFactor);
//	}
//	else
//	{
//		OutAlphaFactor = 1.0f - ( 0.5f + ( FMath::Cos(2.0f * PI * InAnimationProgress) * 0.5f ) );
//		OutPulseFactor = 0.5f + ( FMath::Cos(2.0f * PI * InAnimationProgress) * 0.5f );
//		OutShadowTint = FLinearColor(1.0f, 1.0f, 0.0f, 1.0f);
//		OutBorderTint = FLinearColor(1.0f, 1.0f, 0.0f, TutorialConstants::MinBorderOpacity + ( ( 1.0f - TutorialConstants::MinBorderOpacity ) * OutAlphaFactor ));
//	}
//}

void STutorialContent::GetAnimationValues(float& OutAlphaFactor, float& OutPulseFactor, FLinearColor& OutShadowTint, FLinearColor& OutBorderTint) const
{
	if (BorderIntroAnimation.IsPlaying())
	{
		OutAlphaFactor = BorderIntroAnimation.GetLerp();
		OutPulseFactor = ( 1.0f - OutAlphaFactor ) * 50.0f;
		OutShadowTint = FLinearColor(1.0f, 1.0f, 0.0f, OutAlphaFactor);
		OutBorderTint = FLinearColor(1.0f, 1.0f, 0.0f, OutAlphaFactor * OutAlphaFactor);
	}
	else
	{
		float PulseAnimationProgress = BorderPulseAnimation.GetLerp();
		OutAlphaFactor = 1.0f - ( 0.5f + ( FMath::Cos(2.0f * PI * PulseAnimationProgress) * 0.5f ) );
		OutPulseFactor = 0.5f + ( FMath::Cos(2.0f * PI * PulseAnimationProgress) * 0.5f );
		OutShadowTint = FLinearColor(1.0f, 1.0f, 0.0f, 1.0f);
		OutBorderTint = FLinearColor(1.0f, 1.0f, 0.0f, TutorialConstants::MinBorderOpacity + ( ( 1.0f - TutorialConstants::MinBorderOpacity ) * OutAlphaFactor ));
	}
}

int32 STutorialContent::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	CachedContentGeometry = AllottedGeometry;
	CachedContentGeometry.AppendTransform(FSlateLayoutTransform(OutDrawElements.GetPaintWindow()->GetPositionInScreen()));

	if(bIsVisible && Anchor.Type != ETutorialAnchorIdentifier::None && Anchor.bDrawHighlight)
	{
		float AlphaFactor;
		float PulseFactor;
		FLinearColor ShadowTint;
		FLinearColor BorderTint;
		GetAnimationValues(AlphaFactor, PulseFactor, ShadowTint, BorderTint);

		const FSlateBrush* ShadowBrush = FEditorTutorialStyle::Get().GetBrush(TEXT("Tutorials.Shadow"));
		const FSlateBrush* BorderBrush = FEditorTutorialStyle::Get().GetBrush(TEXT("Tutorials.Highlight.Border"));
					
		const FGeometry& WidgetGeometry = CachedGeometry;
		const FVector2D WindowSize = OutDrawElements.GetPaintWindow()->GetSizeInScreen();

		// We should be clipped by the window size, not our containing widget, as we want to draw outside the widget
		FSlateRect WindowClippingRect(0.0f, 0.0f, WindowSize.X, WindowSize.Y);

		FPaintGeometry ShadowGeometry((FVector2D(WidgetGeometry.AbsolutePosition) - FVector2D(ShadowBrush->Margin.Left, ShadowBrush->Margin.Top) * ShadowBrush->ImageSize * WidgetGeometry.Scale * TutorialConstants::ShadowScale),
										((WidgetGeometry.GetLocalSize() * WidgetGeometry.Scale) + (FVector2D(ShadowBrush->Margin.Right * 2.0f, ShadowBrush->Margin.Bottom * 2.0f) * ShadowBrush->ImageSize * WidgetGeometry.Scale * TutorialConstants::ShadowScale)),
										WidgetGeometry.Scale * TutorialConstants::ShadowScale);
		// draw highlight shadow
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, ShadowGeometry, ShadowBrush, ESlateDrawEffect::None, ShadowTint);

		FVector2D PulseOffset = FVector2D(PulseFactor * TutorialConstants::MaxBorderOffset, PulseFactor * TutorialConstants::MaxBorderOffset);

		FVector2D BorderPosition = (FVector2D(WidgetGeometry.AbsolutePosition) - ((FVector2D(BorderBrush->Margin.Left, BorderBrush->Margin.Top) * BorderBrush->ImageSize * WidgetGeometry.Scale) + PulseOffset));
		FVector2D BorderSize = ((WidgetGeometry.Size * WidgetGeometry.Scale) + (PulseOffset * 2.0f) + (FVector2D(BorderBrush->Margin.Right * 2.0f, BorderBrush->Margin.Bottom * 2.0f) * BorderBrush->ImageSize * WidgetGeometry.Scale));

		FPaintGeometry BorderGeometry(BorderPosition, BorderSize, WidgetGeometry.Scale);

		// draw highlight border
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, BorderGeometry, BorderBrush, ESlateDrawEffect::None, BorderTint);
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply STutorialContent::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!bIsStandalone && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, HandleGetMenuContent(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

/** Helper function to generate title widget, if any */
static TSharedRef<SWidget> GetStageTitle(const FExcerpt& InExcerpt, int32 InCurrentExcerptIndex)
{
	// First try for unadorned 'StageTitle'
	FString VariableName(TEXT("StageTitle"));
	const FString* VariableValue = InExcerpt.Variables.Find(VariableName);
	if(VariableValue != NULL)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(*VariableValue))
			.TextStyle(FEditorTutorialStyle::Get(), "Tutorials.CurrentExcerpt");
	}

	// Then try 'StageTitle<StageNum>'
	VariableName = FString::Printf(TEXT("StageTitle%d"), InCurrentExcerptIndex + 1);
	VariableValue = InExcerpt.Variables.Find(VariableName);
	if(VariableValue != NULL)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(*VariableValue))
			.TextStyle(FEditorTutorialStyle::Get(), "Tutorials.CurrentExcerpt");
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> STutorialContent::GenerateContentWidget(const FTutorialContent& InContent, TSharedPtr<IDocumentationPage>& OutDocumentationPage, const TAttribute<FText>& InHighlightText, bool bAutoWrapText, float WrapTextAt)
{
	// Style for the documentation
	static FDocumentationStyle DocumentationStyle;
	DocumentationStyle
		.ContentStyle(TEXT("Tutorials.Content.Text"))
		.BoldContentStyle(TEXT("Tutorials.Content.TextBold"))
		.NumberedContentStyle(TEXT("Tutorials.Content.Text"))
		.Header1Style(TEXT("Tutorials.Content.HeaderText1"))
		.Header2Style(TEXT("Tutorials.Content.HeaderText2"))
		.HyperlinkStyle(TEXT("Tutorials.Content.Hyperlink"))
		.HyperlinkTextStyle(TEXT("Tutorials.Content.HyperlinkText"))
		.SeparatorStyle(TEXT("Tutorials.Separator"));

	OutDocumentationPage = nullptr;

	switch(InContent.Type)
	{
	case ETutorialContent::Text:
		{
			TSharedRef<STextBlock> TextBlock = SNew(STextBlock)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.AutoWrapText(bAutoWrapText)
				.Text(InContent.Text)
				.TextStyle(FEditorTutorialStyle::Get(), "Tutorials.Content")
				.HighlightText(InHighlightText)
				.HighlightColor(FEditorTutorialStyle::Get().GetColor("Tutorials.Browser.HighlightTextColor"));

			if(!bAutoWrapText)
			{
				TextBlock->SetWrapTextAt(WrapTextAt);
			}

			return TextBlock;
		}

	case ETutorialContent::UDNExcerpt:
		if (IDocumentation::Get()->PageExists(InContent.Content))
		{
			OutDocumentationPage = IDocumentation::Get()->GetPage(InContent.Content, TSharedPtr<FParserConfiguration>(), DocumentationStyle);
			FExcerpt Excerpt;
			if(OutDocumentationPage->GetExcerpt(InContent.ExcerptName, Excerpt) && OutDocumentationPage->GetExcerptContent(Excerpt))
			{
				return SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoHeight()
					.Padding(0.0f, 0.0f, 0.0f, 5.0f)
					[
						GetStageTitle(Excerpt, 0)
					]
					+SVerticalBox::Slot()
					.HAlign(HAlign_Fill)
					.AutoHeight()
					[
						Excerpt.Content.ToSharedRef()
					];
			}
		}
		break;

	case ETutorialContent::RichText:
		{
			TArray< TSharedRef< class ITextDecorator > > Decorators;
			const bool bForEditing = false;
			FTutorialText::GetRichTextDecorators(bForEditing, Decorators);

			TSharedRef<SRichTextBlock> TextBlock = SNew(SRichTextBlock)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.TextStyle(FEditorTutorialStyle::Get(), "Tutorials.Content.Text")
					.DecoratorStyleSet(&FAppStyle::Get())
					.Decorators(Decorators)
					.Text(InContent.Text)
					.AutoWrapText(bAutoWrapText)
					.WrapTextAt(WrapTextAt)
					.Margin(4)
					.LineHeightPercentage(1.1f)
					.HighlightText(InHighlightText);

			return TextBlock;
		}
	}

	return SNullWidget::NullWidget;
}


FVector2D STutorialContent::GetPosition() const
{
	bool bNonVisibleWidgetBound = bAllowNonWidgetContent && !bIsVisible && Anchor.Type == ETutorialAnchorIdentifier::NamedWidget;
	if(bNonVisibleWidgetBound)
	{
		if(OnWasWidgetDrawn.IsBound())
		{
			bNonVisibleWidgetBound &= !OnWasWidgetDrawn.Execute(Anchor.WrapperIdentifier);
		}
	}

	if(bNonVisibleWidgetBound)
	{
		// fallback: center on cached window
		return FVector2D((CachedWindowSize.X * 0.5f) - (ContentWidget->GetDesiredSize().X * 0.5f),
						 (CachedWindowSize.Y * 0.5f) - (ContentWidget->GetDesiredSize().Y * 0.5f));
	}
	else
	{
		float XOffset = 0.0f;
		switch(HorizontalAlignment.Get())
		{
		case HAlign_Left:
			XOffset = -(ContentWidget->GetDesiredSize().X - ContentOffset);
			break;
		default:
		case HAlign_Fill:
		case HAlign_Center:
			XOffset = (CachedGeometry.GetLocalSize().X * 0.5f) - (ContentWidget->GetDesiredSize().X * 0.5f);
			break;
		case HAlign_Right:
			XOffset = CachedGeometry.GetLocalSize().X - ContentOffset;
			break;
		}

		XOffset += WidgetOffset.Get().X;

		float YOffset = 0.0f;
		switch(VerticalAlignment.Get())
		{
		case VAlign_Top:
			YOffset = -(ContentWidget->GetDesiredSize().Y - ContentOffset);
			break;
		default:
		case VAlign_Fill:
		case VAlign_Center:
			YOffset = (CachedGeometry.GetLocalSize().Y * 0.5f) - (ContentWidget->GetDesiredSize().Y * 0.5f);
			break;
		case VAlign_Bottom:
			YOffset = (CachedGeometry.GetLocalSize().Y - ContentOffset);
			break;
		}

		YOffset += WidgetOffset.Get().Y;

		// now build & clamp to area
		float WidgetX = CachedGeometry.GetAbsolutePosition().X;
		float WidgetY = CachedGeometry.GetAbsolutePosition().Y;
		float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WidgetX, WidgetY);
		FVector2D BaseOffset = FVector2D(WidgetX / DPIScale + XOffset, WidgetY / DPIScale + YOffset);
		BaseOffset.X = FMath::Clamp(BaseOffset.X, 0.0f, CachedWindowSize.X - ContentWidget->GetDesiredSize().X);
		BaseOffset.Y = FMath::Clamp(BaseOffset.Y, 0.0f, CachedWindowSize.Y - ContentWidget->GetDesiredSize().Y);
		return BaseOffset;
	}
}

FVector2D STutorialContent::GetSize() const
{
	return ContentWidget->GetDesiredSize();
}

FReply STutorialContent::OnCloseButtonClicked()
{
	OnClosed.ExecuteIfBound();

	return FReply::Handled();
}

EVisibility STutorialContent::GetCloseButtonVisibility() const
{
	return bIsStandalone ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STutorialContent::GetMenuButtonVisibility() const
{
	return !bIsStandalone ? EVisibility::Visible : EVisibility::Collapsed;
}

void STutorialContent::HandlePaintNamedWidget(TSharedRef<SWidget> InWidget, const FGeometry& InGeometry)
{
	switch(Anchor.Type)
	{
	case ETutorialAnchorIdentifier::NamedWidget:
		{
			TSharedPtr<FTagMetaData> WidgetMetaData = InWidget->GetMetaData<FTagMetaData>();
			if( Anchor.WrapperIdentifier == InWidget->GetTag() ||
				(WidgetMetaData.IsValid() && WidgetMetaData->Tag == Anchor.WrapperIdentifier))
			{
				bIsVisible = true;
				CachedGeometry = InGeometry;

				if (!BorderPulseAnimation.IsPlaying() && Anchor.bDrawHighlight)
				{
					BorderPulseAnimation.Resume();
				}
			}
		}
		break;
	}
}

void STutorialContent::HandleResetNamedWidget()
{
	BorderPulseAnimation.Pause();
	bIsVisible = false;
}

void STutorialContent::HandleCacheWindowSize(const FVector2D& InWindowSize)
{
	CachedWindowSize = InWindowSize;
}

EVisibility STutorialContent::GetVisibility() const
{
	const bool bVisibleWidgetBound = bIsVisible && Anchor.Type == ETutorialAnchorIdentifier::NamedWidget;
	const bool bNonWidgetBound =  Anchor.Type == ETutorialAnchorIdentifier::None;

	// fallback if widget is not drawn - we should display this content anyway
	bool bNonVisibleWidgetBound = bAllowNonWidgetContent && !bIsVisible && Anchor.Type == ETutorialAnchorIdentifier::NamedWidget;
	if(bNonVisibleWidgetBound)
	{
		if(OnWasWidgetDrawn.IsBound())
		{
			bNonVisibleWidgetBound &= !OnWasWidgetDrawn.Execute(Anchor.WrapperIdentifier);
		}	
	}

	return (bVisibleWidgetBound || bNonWidgetBound || bNonVisibleWidgetBound) ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

FSlateColor STutorialContent::GetBackgroundColor() const
{
	// note cant use IsHovered() here because our widget is SelfHitTestInvisible
	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
	return CachedContentGeometry.IsUnderLocation(CursorPos) ? FEditorTutorialStyle::Get().GetColor("Tutorials.Content.Color.Hovered") : FEditorTutorialStyle::Get().GetColor("Tutorials.Content.Color");
}

float STutorialContent::GetAnimatedZoom() const
{
	if(ContentIntroAnimation.IsPlaying() && FSlateApplication::Get().IsRunningAtTargetFrameRate())
	{
		FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
		return 0.75f + (0.25f * IntroTutorials.GetIntroCurveValue(ContentIntroAnimation.GetLerp()));
	}
	else
	{
		return 1.0f;
	}
}

float STutorialContent::GetInverseAnimatedZoom() const
{
	return 1.0f / GetAnimatedZoom();
}

TSharedRef<SWidget> STutorialContent::HandleGetMenuContent()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, TSharedPtr<const FUICommandList>());

	MenuBuilder.BeginSection(TEXT("Tutorial Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ExitLabel", "Exit"),
			LOCTEXT("ExitTooltip", "Quit this tutorial. You can find it again in the tutorials browser, reached from the Help menu."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STutorialContent::HandleExitSelected),
				FCanExecuteAction()
			)
		);

		if(IsNextEnabled.Get())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("NextLabel", "Next"),
				GetNextButtonTooltip(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STutorialContent::HandleNextSelected),
					FCanExecuteAction()
				)
			);
		}

		if(IsBackEnabled.Get())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("BackLabel", "Back"),
				LOCTEXT("BackTooltip", "Go back to the previous stage of this tutorial."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STutorialContent::HandleBackSelected),
					FCanExecuteAction()
				)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("RestartLabel", "Restart"),
				LOCTEXT("RestartTooltip", "Start this tutorial again from the beginning."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STutorialContent::HandleRestartSelected),
					FCanExecuteAction()
				)
			);
		}

		if(IsHomeEnabled.Get())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("OpenBrowserLabel", "More Tutorials..."),
				LOCTEXT("OpenBrowserTooltip", "Open the tutorial browser to find more tutorials."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &STutorialContent::HandleBrowseSelected),
					FCanExecuteAction()
				)
			);	
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void STutorialContent::HandleExitSelected()
{
	OnClosed.ExecuteIfBound();
}

void STutorialContent::HandleNextSelected()
{
	OnNextClicked.ExecuteIfBound();
}

void STutorialContent::HandleBackSelected()
{
	OnBackClicked.ExecuteIfBound();
}

void STutorialContent::HandleRestartSelected()
{
	if(Tutorial.IsValid())
	{
		FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
		IntroTutorials.LaunchTutorial(Tutorial.Get(), IIntroTutorials::ETutorialStartType::TST_RESTART, FSlateApplication::Get().FindWidgetWindow(AsShared()));

		if( FEngineAnalytics::IsAvailable() && Tutorial.IsValid() )
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TutorialAsset"), FIntroTutorials::AnalyticsEventNameFromTutorial(Tutorial.Get())));

			FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.Restarted"), EventAttributes );
		}
	}
}

void STutorialContent::HandleBrowseSelected()
{
	if( FEngineAnalytics::IsAvailable() && Tutorial.IsValid())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("FromTutorial"), FIntroTutorials::AnalyticsEventNameFromTutorial(Tutorial.Get())));

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.OpenedBrowser"), EventAttributes );
	}

	OnHomeClicked.ExecuteIfBound();
}

FReply STutorialContent::HandleNextClicked()
{
	if(IsNextEnabled.Get())
	{
		OnNextClicked.ExecuteIfBound();
	}
	else
	{
		OnHomeClicked.ExecuteIfBound();
	}
	
	return FReply::Handled();
}

FReply STutorialContent::HandleBackButtonClicked()
{
	if (IsBackEnabled.Get())
	{
		OnBackClicked.ExecuteIfBound();
	}

	return FReply::Handled();
}

const FSlateBrush* STutorialContent::GetNextButtonBrush() const
{
	if(IsNextEnabled.Get())
	{
		return FEditorTutorialStyle::Get().GetBrush("Tutorials.Navigation.NextButton");
	}
	else
	{
		return FEditorTutorialStyle::Get().GetBrush("Tutorials.Navigation.HomeButton");
	}
}

FText STutorialContent::GetNextButtonTooltip() const
{
	if(IsNextEnabled.Get())
	{
		return LOCTEXT("NextButtonTooltip", "Go to the next stage of this tutorial.");
	}
	else
	{
		return LOCTEXT("HomeButtonTooltip", "This tutorial is complete. Open the tutorial browser to find more tutorials.");
	}
}

FText STutorialContent::GetNextButtonLabel() const
{
	if(!NextButtonText.Get().IsEmpty())
	{
		return NextButtonText.Get();
	}
	else
	{
		if(IsNextEnabled.Get())
		{
			return LOCTEXT("DefaultNextButtonLabel", "Next");
		}
		else
		{
			return LOCTEXT("DefaultHomeButtonLabel", "Home");
		}
	}
}

const FSlateBrush* STutorialContent::GetNextButtonBorder() const
{
	return NextButton->IsHovered() ? &FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.NavigationButton").Hovered : &FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.NavigationButton").Normal;
}

const FSlateBrush* STutorialContent::GetBackButtonBrush() const
{
	if (IsBackEnabled.Get())
	{
		return FEditorTutorialStyle::Get().GetBrush("Tutorials.Navigation.BackButton");
	}
	return FAppStyle::GetDefaultBrush();
}

EVisibility STutorialContent::GetBackButtonVisibility() const
{
	return IsBackEnabled.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText STutorialContent::GetBackButtonTooltip() const
{
	if (IsBackEnabled.Get())
	{
		return LOCTEXT("BackButtonTooltip", "Go to the previous stage of this tutorial.");
	}
	return FText::GetEmpty();
}

FText STutorialContent::GetBackButtonLabel() const
{
	if (!BackButtonText.Get().IsEmpty())
	{
		return BackButtonText.Get();
	}
	else
	{
		if (IsBackEnabled.Get())
		{
			return LOCTEXT("DefaultBackButtonLabel", "Back");
		}	
		
	}
	return FText::GetEmpty();
}

const FSlateBrush* STutorialContent::GetBackButtonBorder() const
{
	return BackButton->IsHovered() ? &FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.NavigationBackButton").Hovered : &FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.NavigationBackButton").Normal;
}

FReply STutorialContent::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Mouse back and forward buttons traverse history
	if ( MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		if(IsBackEnabled.Get())
		{
			OnBackClicked.ExecuteIfBound();
			return FReply::Handled();
		}
	}
	else if ( MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		if(IsNextEnabled.Get())
		{
			OnNextClicked.ExecuteIfBound();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply STutorialContent::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	// Mouse back and forward buttons traverse history
	if ( InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		if(IsBackEnabled.Get())
		{
			OnBackClicked.ExecuteIfBound();
			return FReply::Handled();
		}
	}
	else if ( InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		if(IsNextEnabled.Get())
		{
			OnNextClicked.ExecuteIfBound();
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
