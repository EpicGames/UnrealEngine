// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "EditorTutorial.h"
#include "STutorialOverlay.h"

class FPaintArgs;
class FSlateWindowElementList;
class IDocumentationPage;
struct FSlateBrush;

/**
 * The widget which displays 'floating' content 
 */
class STutorialContent : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( STutorialContent )
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
		_IsStandalone = false;
	}

	/** Alignment of content relative to widget, note "Fill" is not supported */
	SLATE_ATTRIBUTE(EVerticalAlignment, VAlign)

	/** Alignment of content relative to widget, note "Fill" is not supported */
	SLATE_ATTRIBUTE(EHorizontalAlignment, HAlign)

	/** Offset form the widget we annotate */
	SLATE_ATTRIBUTE(FVector2D, Offset)

	/** Whether this a standalone widget (with its own close button) or part of a group of other widgets, paired with tutorial navigation */
	SLATE_ARGUMENT(bool, IsStandalone)

	/** Delegate fired when the close button is clicked */
	SLATE_EVENT(FSimpleDelegate, OnClosed)

	/** Delegate fired when the back button is clicked */
	SLATE_EVENT(FSimpleDelegate, OnBackClicked)

	/** Delegate fired when the home button is clicked */
	SLATE_EVENT(FSimpleDelegate, OnHomeClicked)

	/** Delegate fired when the next button is clicked */
	SLATE_EVENT(FSimpleDelegate, OnNextClicked)

	/** Attribute controlling enabled state of back functionality */
	SLATE_ATTRIBUTE(bool, IsBackEnabled)

	/** Attribute controlling enabled state of home functionality */
	SLATE_ATTRIBUTE(bool, IsHomeEnabled)

	/** Attribute controlling enabled state of next functionality */
	SLATE_ATTRIBUTE(bool, IsNextEnabled)

	/** Where text should be wrapped */
	SLATE_ARGUMENT(float, WrapTextAt)

	/** Anchor if required */
	SLATE_ARGUMENT(FTutorialContentAnchor, Anchor)

	/** Whether we can show full window content in this overlay (i.e. in the same window as the navigation controls) */
	SLATE_ARGUMENT(bool, AllowNonWidgetContent)

	/** Delegate for querying whether a widget was drawn */
	SLATE_EVENT(FOnWasWidgetDrawn, OnWasWidgetDrawn)

	/** Text to display on next/home button */
	SLATE_ATTRIBUTE(FText, NextButtonText)

	/** Text to display on back button */
	SLATE_ATTRIBUTE(FText, BackButtonText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEditorTutorial* InTutorial, const FTutorialContent& InContent);

	/** SWidget implementation */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Helper function to generate widgets from an FTutorialContent struct */
	static TSharedRef<SWidget> GenerateContentWidget(const FTutorialContent& InContent, TSharedPtr<IDocumentationPage>& OutDocumentationPage, const TAttribute<FText>& InHighlightText = TAttribute<FText>(), bool bAutoWrapText = true, float WrapTextAt = 0.0f);

	/** Handle repositioning the widget */
	FVector2D GetPosition() const;

	/** Handle resizing the widget */
	FVector2D GetSize() const;

	/** Delegate handler called back from the overlay paint routines to flag whether we should paint as well (i.e. if this widget content is highlighted for the current stage) */
	void HandlePaintNamedWidget(TSharedRef<SWidget> InWidget, const FGeometry& InGeometry);

	/** Called back from the overlay paint routines to reset the flag we check for painting with */
	void HandleResetNamedWidget();

	/** Handle caching window size - called back from overlay paint routine */
	void HandleCacheWindowSize(const FVector2D& InWindowSize);

private:

	/** Get the visibility of this content */
	EVisibility GetVisibility() const;

	/** Handle close button clicked - forward to delegate */
	FReply OnCloseButtonClicked();

	/** Get close button visibility - varies depending on whether we are standalone or not */
	EVisibility GetCloseButtonVisibility() const;

	/** Get menu button visibility - varies depending on whether we are standalone or not */
	EVisibility GetMenuButtonVisibility() const;

	/** Alter the background color depending on hover state */
	FSlateColor GetBackgroundColor() const;

	/** Get zoom level padding for content (animated for intro) */
	float GetAnimatedZoom() const;
	
	/** Get inverse zoom level padding for content - needed because rich text content doesnt scale well */
	float GetInverseAnimatedZoom() const;

	/** Get the content for the navigation menu */
	TSharedRef<SWidget> HandleGetMenuContent();

	/** Delegate handler for exiting the tutorial */
	void HandleExitSelected();

	/** Delegate handler for going to the previous stage of the tutorial (From dropdown menu) */
	void HandleBackSelected();
	
	/** Delegate handler for going to the next stage of the tutorial (From dropdown menu) */
	void HandleNextSelected();

	/** Delegate handler for restarting the tutorial */
	void HandleRestartSelected();

	/** Delegate handler for exiting the tutorial to the browser */
	void HandleBrowseSelected();

	/** Delegate handler for going to the next stage of the tutorial (From button) */
	FReply HandleNextClicked();

	/** Delegate handler for going to the previous stage of the tutorial (From button) */
	FReply HandleBackButtonClicked();

	/** Delegate handler allowing us to change the brush of the 'next' button depending on context */
	const FSlateBrush* GetNextButtonBrush() const;

	/** Delegate handler allowing us to change the tootlip of the 'next' button depending on context */
	FText GetNextButtonTooltip() const;

	/** Change next button color based on hover state */
	FText GetNextButtonLabel() const;

	/** We need to override the border ourselves, rather than let the button handle it, as we are using a larger apparent hitbox */
	const FSlateBrush* GetNextButtonBorder() const;

	/** Helper to determine the proper animation values for the border pulse */
	void GetAnimationValues(float& OutAlphaFactor, float& OutPulseFactor, FLinearColor& OutShadowTint, FLinearColor& OutBorderTint) const;

	/** Delegate handler allowing us to change the brush of the 'back' button depending on context */
	const FSlateBrush* GetBackButtonBrush() const;

	/** Delegate handler allowing us to change the tootlip of the 'back' button depending on context */
	FText GetBackButtonTooltip() const;

	/** Change back button color based on hover state */
	FText GetBackButtonLabel() const;

	/** We need to override the border ourselves, rather than let the button handle it, as we are using a larger apparent hitbox */
	const FSlateBrush* GetBackButtonBorder() const;

	/* Get the visibilty of the back button */
	EVisibility GetBackButtonVisibility() const;

private:

	/** Copy of the window size we were last draw at */
	FVector2D CachedWindowSize;

	/** Copy of the geometry our widget was last drawn with */
	FGeometry CachedGeometry;

	/** Copy of the geometry our content was last drawn with */
	mutable FGeometry CachedContentGeometry;

	/** Container for widget content */
	TSharedPtr<SWidget> ContentWidget;

	/** Alignment of content relative to widget, note "Fill" is not supported */
	TAttribute<EVerticalAlignment> VerticalAlignment;

	/** Alignment of content relative to widget, note "Fill" is not supported */
	TAttribute<EHorizontalAlignment> HorizontalAlignment;

	/** Offset form the widget we annotate */
	TAttribute<FVector2D> WidgetOffset;

	/** Copy of the anchor for this tutorial content */
	FTutorialContentAnchor Anchor; 

	/** Whether this a standalone widget (with its own close button) or part of a group of other widgets, paired with tutorial navigation */
	bool bIsStandalone;

	/** Whether this overlay is currently visible */
	bool bIsVisible;

	/** Delegate fired when the close button is clicked */
	FSimpleDelegate OnClosed;

	/** Delegate fired when the next button is clicked */
	FSimpleDelegate OnNextClicked;

	/** Delegate fired when the home button is clicked */
	FSimpleDelegate OnHomeClicked;

	/** Delegate fired when the back button is clicked */
	FSimpleDelegate OnBackClicked;

	/** Attribute controlling enabled state of back functionality */
	TAttribute<bool> IsBackEnabled;

	/** Attribute controlling enabled state of home functionality */
	TAttribute<bool> IsHomeEnabled;

	/** Attribute controlling enabled state of next functionality */
	TAttribute<bool> IsNextEnabled;

	/** Animation curves for displaying border */
	FCurveSequence BorderPulseAnimation;
	FCurveSequence BorderIntroAnimation;

	/** Animation curve for displaying content */
	FCurveSequence ContentIntroAnimation;

	/** Documentation page reference to use if we are displaying a UDN doc - we need this otherwise the page will be freed */
	TSharedPtr<IDocumentationPage> DocumentationPage;

	/** The tutorial we are referencing */
	TWeakObjectPtr<UEditorTutorial> Tutorial;

	/** Next button widget */
	TSharedPtr<SWidget> NextButton;

	/** Back button widget */
	TSharedPtr<SWidget> BackButton;

	/** Whether we can show full window content in this overlay (i.e. in the same window as the navigation controls) */
	bool bAllowNonWidgetContent;

	/** Delegate for querying whether a widget was drawn */
	FOnWasWidgetDrawn OnWasWidgetDrawn;

	/** Text for next/home button */
	TAttribute<FText> NextButtonText;

	/** Text for next/home button */
	TAttribute<FText> BackButtonText;
};
