// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialEditableText.h"
#include "EditorTutorialStyle.h"
#include "Misc/PackageName.h"
#include "SlateOptMacros.h"
#include "Framework/Text/SlateImageRun.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Text/ITextDecorator.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "EditorDirectories.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "TutorialText.h"
#include "TutorialImageDecorator.h"
#include "DesktopPlatformModule.h"

#define LOCTEXT_NAMESPACE "STutorialEditableText"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STutorialEditableText::Construct(const FArguments& InArgs)
{
	OnTextChanged = InArgs._OnTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;

	bNewHyperlink = true;

	// Setup text styles
	StylesAndNames.Add(MakeShareable(new FTextStyleAndName(TEXT("Tutorials.Content.Text"), LOCTEXT("NormalTextDesc", "Normal"))));
	StylesAndNames.Add(MakeShareable(new FTextStyleAndName(TEXT("Tutorials.Content.TextBold"), LOCTEXT("BoldTextDesc", "Bold"))));
	StylesAndNames.Add(MakeShareable(new FTextStyleAndName(TEXT("Tutorials.Content.HeaderText2"), LOCTEXT("Header2TextDesc", "Header 2"))));
	StylesAndNames.Add(MakeShareable(new FTextStyleAndName(TEXT("Tutorials.Content.HeaderText1"), LOCTEXT("Header1TextDesc", "Header 1"))));
	ActiveStyle = StylesAndNames[0];
	
	HyperlinkStyle = MakeShareable(new FTextStyleAndName(TEXT("Tutorials.Content.HyperlinkText"), LOCTEXT("HyperlinkTextDesc", "Hyperlink")));
	StylesAndNames.Add(HyperlinkStyle);

	TSharedRef<FRichTextLayoutMarshaller> RichTextMarshaller = FRichTextLayoutMarshaller::Create(
		TArray<TSharedRef<ITextDecorator>>(), 
		&FAppStyle::Get()
		);


	TArray<TSharedRef<class ITextDecorator>> Decorators;
	const bool bForEditing = true;
	FTutorialText::GetRichTextDecorators(bForEditing, Decorators);

	for(auto& Decorator : Decorators)
	{
		RichTextMarshaller->AppendInlineDecorator(Decorator);
	}

	CurrentHyperlinkType = FTutorialText::GetHyperLinkDescs()[0];

	this->ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		[
			SAssignNew(RichEditableTextBox, SMultiLineEditableTextBox)
			.Font(FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>("Tutorials.Content.Text").Font)
			.Text(InArgs._Text)
			.OnTextChanged(this, &STutorialEditableText::HandleRichEditableTextChanged)
			.OnTextCommitted(this, &STutorialEditableText::HandleRichEditableTextCommitted)
			.OnCursorMoved(this, &STutorialEditableText::HandleRichEditableTextCursorMoved)
			.Marshaller(RichTextMarshaller)
			.ClearTextSelectionOnFocusLoss(false)
			.AutoWrapText(true)
			.Margin(4)
			.LineHeightPercentage(1.1f)
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
		[
			SNew(SBorder)
			.Visibility(this, &STutorialEditableText::GetToolbarVisibility)
			.BorderImage(FEditorTutorialStyle::Get().GetBrush("TutorialEditableText.RoundedBackground"))
			.Padding(FMargin(4))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(FontComboBox, SComboBox<TSharedPtr<FTextStyleAndName>>)
					.ComboBoxStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.ComboBox")
					.OptionsSource(&StylesAndNames)
					.OnSelectionChanged(this, &STutorialEditableText::OnActiveStyleChanged)
					.OnGenerateWidget(this, &STutorialEditableText::GenerateStyleComboEntry)
					.ContentPadding(0.0f)
					.InitiallySelectedItem(nullptr)
					[
						SNew(SBox)
						.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
						.MinDesiredWidth(100.0f)
						[
							SNew(STextBlock)
							.Text(this, &STutorialEditableText::GetActiveStyleName)
						]
					]
				]

				+SHorizontalBox::Slot()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.AutoWidth()
				[
					SAssignNew(HyperlinkComboButton, SComboButton)
					.ToolTipText(LOCTEXT("HyperlinkButtonTooltip", "Insert or Edit Hyperlink"))
					.ComboButtonStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.ComboButton")
					.OnComboBoxOpened(this, &STutorialEditableText::HandleHyperlinkComboOpened)
					.IsEnabled(this, &STutorialEditableText::IsHyperlinkComboEnabled)
					.ContentPadding(1.0f)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FEditorTutorialStyle::Get().GetBrush("TutorialEditableText.Toolbar.HyperlinkImage"))
					]
					.MenuContent()
					[
						SNew(SGridPanel)
						.FillColumn(1, 1.0f)

						+SGridPanel::Slot(0, 0)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.Padding(FMargin(2.0f))
						[
							SNew(STextBlock)
							.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
							.Text(LOCTEXT("HyperlinkNameLabel", "Name:"))
						]
						+SGridPanel::Slot(1, 0)
						.Padding(FMargin(2.0f))
						[
							SNew(SBox)
							.WidthOverride(300)
							[
								SAssignNew(HyperlinkNameTextBlock, STextBlock)
								.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
							]
						]

						+SGridPanel::Slot(0, 1)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.Padding(FMargin(2.0f))
						[
							SNew(STextBlock)
							.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
							.Text(LOCTEXT("HyperlinkURLLabel", "URL:"))
						]
						+SGridPanel::Slot(1, 1)
						.Padding(FMargin(2.0f))
						[
							SNew(SBox)
							.WidthOverride(300)
							[
								SAssignNew(HyperlinkURLTextBox, SEditableTextBox)
							]
						]

						+SGridPanel::Slot(0, 2)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.Padding(FMargin(2.0f))
						[
							SNew(STextBlock)
							.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
							.Text(LOCTEXT("HyperlinkTypeLabel", "Type:"))
						]

						+SGridPanel::Slot(1, 2)
						.Padding(FMargin(2.0f))
						.VAlign(VAlign_Center)
						.ColumnSpan(2)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SComboBox<TSharedPtr<FHyperlinkTypeDesc>>)
								.OptionsSource(&FTutorialText::GetHyperLinkDescs())
								.ComboBoxStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.ComboBox")
								.OnSelectionChanged(this, &STutorialEditableText::OnActiveHyperlinkChanged)
								.OnGenerateWidget(this, &STutorialEditableText::GenerateHyperlinkComboEntry)
								.ContentPadding(0.0f)
								.InitiallySelectedItem(FTutorialText::GetHyperLinkDescs()[0])
								.Content()
								[
									SNew(SBox)
									.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
									.MinDesiredWidth(100.0f)
									[
										SNew(STextBlock)
										.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
										.Text(this, &STutorialEditableText::GetActiveHyperlinkName)
										.ToolTipText(this, &STutorialEditableText::GetActiveHyperlinkTooltip)
									]
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(5.0f, 0.0f, 0.0f, 0.0f)
							[
								SNew(SCheckBox)
								.ToolTipText(LOCTEXT("OpenAssetTooltip", "Should this link open the asset or just highlight it in the content browser?"))
								.Visibility(this, &STutorialEditableText::GetOpenAssetVisibility)
								.IsChecked(this, &STutorialEditableText::IsOpenAssetChecked)
								.OnCheckStateChanged(this, &STutorialEditableText::HandleOpenAssetCheckStateChanged)
								[
									SNew(STextBlock)
									.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
									.Text(LOCTEXT("OpenAssetLabel", "Open Asset"))
								]
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(5.0f, 0.0f, 0.0f, 0.0f)
							[
								SAssignNew(UDNExcerptTextBox, SEditableTextBox)
								.HintText(LOCTEXT("ExcerptHint", "Excerpt"))
								.ToolTipText(LOCTEXT("ExcerptAssetTooltip", "Enter the excerpt that should be used for this link's rich tooltip"))
								.Visibility(this, &STutorialEditableText::GetExcerptVisibility)
							]
							+SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Right)
							[
								SNew(SButton)
								.ButtonStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Button")
								.OnClicked(this, &STutorialEditableText::HandleInsertHyperLinkClicked)
								[
									SNew(STextBlock)
									.TextStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Text")
									.Text(this, &STutorialEditableText::GetHyperlinkButtonText)
								]
							]
						]
					]
				]

				+SHorizontalBox::Slot()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.AutoWidth()
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("ImageButtonTooltip", "Insert Image"))
					.ButtonStyle(FEditorTutorialStyle::Get(), "TutorialEditableText.Toolbar.Button")
					.OnClicked(this, &STutorialEditableText::HandleImageButtonClicked)
					.ContentPadding(1.0f)
					.Content()
					[
						SNew(SImage)
						.Image(FEditorTutorialStyle::Get().GetBrush("TutorialEditableText.Toolbar.ImageImage"))
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION
	
void STutorialEditableText::HandleRichEditableTextChanged(const FText& Text)
{
	OnTextChanged.ExecuteIfBound(Text);
}

void STutorialEditableText::HandleRichEditableTextCommitted(const FText& Text, ETextCommit::Type Type)
{
	OnTextCommitted.ExecuteIfBound(Text, Type);
}

static bool AreRunsTheSame(const TArray<TSharedRef<const IRun>>& Runs)
{
	if(Runs.Num() == 0)
	{
		return false;
	}

	TSharedRef<const IRun> FirstRun = Runs[0];
	for(const auto& Run : Runs)
	{
		if(Run != FirstRun)
		{
			if(Run->GetRunInfo().Name != FirstRun->GetRunInfo().Name)
			{
				return false;
			}

			for(const auto& MetaData : FirstRun->GetRunInfo().MetaData)
			{
				const FString* FoundMetaData = Run->GetRunInfo().MetaData.Find(MetaData.Key);
				if(FoundMetaData == nullptr || *FoundMetaData != MetaData.Value)
				{
					return false;
				}
			}
		}
	}

	return true;
}

TSharedPtr<const IRun> STutorialEditableText::GetCurrentRun() const
{
	if(!RichEditableTextBox->GetSelectedText().IsEmpty())
	{
		const TArray<TSharedRef<const IRun>> Runs = RichEditableTextBox->GetSelectedRuns();
		if(Runs.Num() == 1 || AreRunsTheSame(Runs))
		{
			return Runs[0];
		}
	}
	else
	{
		return RichEditableTextBox->GetRunUnderCursor();
	}

	return TSharedPtr<const IRun>();
}

void STutorialEditableText::HandleRichEditableTextCursorMoved(const FTextLocation& NewCursorPosition )
{
	TSharedPtr<const IRun> Run = GetCurrentRun();
	
	if(Run.IsValid())
	{
		if(Run->GetRunInfo().Name == TEXT("TextStyle"))
		{
			ActiveStyle = StylesAndNames[0];

			FName StyleName = FTextStyleAndName::GetStyleFromRunInfo(Run->GetRunInfo());
			for(const auto& StyleAndName : StylesAndNames)
			{
				if(StyleAndName->Style == StyleName)
				{
					ActiveStyle = StyleAndName;
					break;
				}
			}
		}
		else if(Run->GetRunInfo().Name == TEXT("a"))
		{
			ActiveStyle = HyperlinkStyle;
		}

		FontComboBox->SetSelectedItem(ActiveStyle);
	}
	else
	{
		FontComboBox->SetSelectedItem(nullptr);
	}
}

FText STutorialEditableText::GetActiveStyleName() const
{
	return ActiveStyle.IsValid() ? ActiveStyle->DisplayName : FText();
}

void STutorialEditableText::OnActiveStyleChanged(TSharedPtr<FTextStyleAndName> NewValue, ESelectInfo::Type SelectionType)
{
	ActiveStyle = NewValue;
	if(SelectionType != ESelectInfo::Direct)
	{
		// style text if it was the user that made this selection
		if(NewValue == HyperlinkStyle)
		{
			HandleHyperlinkComboOpened();
			HyperlinkComboButton->SetIsOpen(true);
		}
		else
		{
			StyleSelectedText();
		}
	}
}

TSharedRef<SWidget> STutorialEditableText::GenerateStyleComboEntry(TSharedPtr<FTextStyleAndName> SourceEntry)
{
	return SNew(SBorder)
		.BorderImage( FAppStyle::Get().GetBrush( "NoBorder" ) )
		.ForegroundColor(FAppStyle::Get().GetSlateColor("InvertedForeground"))
		[
			SNew(STextBlock)
			.Text(SourceEntry->DisplayName)
			.TextStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>(SourceEntry->Style))
		];
}

void STutorialEditableText::StyleSelectedText()
{
	// Apply the current style to the selected text
	// If no text is selected, then a new (empty) run will be inserted with the appropriate style
	if(ActiveStyle.IsValid())
	{
		const FRunInfo RunInfo = ActiveStyle->CreateRunInfo();
		const FTextBlockStyle TextBlockStyle = ActiveStyle->CreateTextBlockStyle();
		RichEditableTextBox->ApplyToSelection(RunInfo, TextBlockStyle);
		FSlateApplication::Get().SetKeyboardFocus(RichEditableTextBox, EFocusCause::SetDirectly);
	}
}

TSharedPtr<FHyperlinkTypeDesc> STutorialEditableText::GetHyperlinkTypeFromId(const FString& InId) const
{
	TSharedPtr<FHyperlinkTypeDesc> FoundType;
	for(const auto& Desc : FTutorialText::GetHyperLinkDescs())
	{
		if(Desc->Id == InId)
		{
			return Desc;
		}
	}

	return FoundType;
}

void STutorialEditableText::HandleHyperlinkComboOpened()
{
	HyperlinkURLTextBox->SetText(FText());
	HyperlinkNameTextBlock->SetText(FText());

	// Read any currently selected text, and use this as the default name of the hyperlink
	FString SelectedText = RichEditableTextBox->GetSelectedText().ToString();
	if(SelectedText.Len() > 0)
	{
		for(int32 SelectedTextIndex = 0; SelectedTextIndex < SelectedText.Len(); ++SelectedTextIndex)
		{
			if(FChar::IsLinebreak(SelectedText[SelectedTextIndex]))
			{
				SelectedText.LeftInline(SelectedTextIndex, EAllowShrinking::No);
				break;
			}
		}
		HyperlinkNameTextBlock->SetText(FText::FromString(SelectedText));
	}

	TSharedPtr<const IRun> Run = GetCurrentRun();
	if(Run.IsValid() && Run->GetRunInfo().Name == TEXT("a"))
	{
		const FString* const URLUnderCursor = Run->GetRunInfo().MetaData.Find(TEXT("href"));
		HyperlinkURLTextBox->SetText((URLUnderCursor) ? FText::FromString(*URLUnderCursor) : FText());

		const FString* const IdUnderCursor = Run->GetRunInfo().MetaData.Find(TEXT("id"));
		CurrentHyperlinkType = IdUnderCursor ? GetHyperlinkTypeFromId(*IdUnderCursor) : FTutorialText::GetHyperLinkDescs()[0];

		FString RunText;
		Run->AppendTextTo(RunText);
		HyperlinkNameTextBlock->SetText(FText::FromString(RunText));
	}	
}

bool STutorialEditableText::IsHyperlinkComboEnabled() const
{
	return ActiveStyle == HyperlinkStyle;
}

FReply STutorialEditableText::HandleInsertHyperLinkClicked()
{
	HyperlinkComboButton->SetIsOpen(false);

	const FText& Name = HyperlinkNameTextBlock->GetText();
	if(!Name.IsEmpty())
	{
		const FText& URL = HyperlinkURLTextBox->GetText();

		// Create the correct meta-information for this run, so that valid source rich-text formatting can be generated for it
		FRunInfo RunInfo(TEXT("a"));
		RunInfo.MetaData.Add(TEXT("id"), CurrentHyperlinkType->Id);
		RunInfo.MetaData.Add(TEXT("href"), URL.ToString());
		RunInfo.MetaData.Add(TEXT("style"), TEXT("Tutorials.Content.Hyperlink"));

		if(CurrentHyperlinkType->Type == EHyperlinkType::Asset)
		{
			RunInfo.MetaData.Add(TEXT("action"), bOpenAsset ? TEXT("edit") : TEXT("select"));
		}
		else if(CurrentHyperlinkType->Type == EHyperlinkType::UDN && !UDNExcerptTextBox->GetText().IsEmpty())
		{
			RunInfo.MetaData.Add(TEXT("excerpt"), UDNExcerptTextBox->GetText().ToString());
		}

		// Create the new run, and then insert it at the cursor position
		TSharedRef<FSlateHyperlinkRun> HyperlinkRun = FSlateHyperlinkRun::Create(
			RunInfo, 
			MakeShareable(new FString(Name.ToString())), 
			FEditorTutorialStyle::Get().GetWidgetStyle<FHyperlinkStyle>(FName(TEXT("Tutorials.Content.Hyperlink"))),
			CurrentHyperlinkType->OnClickedDelegate,
			CurrentHyperlinkType->TooltipDelegate,
			CurrentHyperlinkType->TooltipTextDelegate
			);

		// @todo: if the rich editable text box allowed us to replace a run that we found under the cursor (or returned a non-const
		// instance of it) then we could edit the hyperlink here. This would mean the user does not need to select the whole hyperlink 
		// to edit its URL.
		RichEditableTextBox->InsertRunAtCursor(HyperlinkRun);
	}

	return FReply::Handled();
}

EVisibility STutorialEditableText::GetToolbarVisibility() const
{
	return FontComboBox->IsOpen() || HyperlinkComboButton->IsOpen() || HasKeyboardFocus() || HasFocusedDescendants() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText STutorialEditableText::GetHyperlinkButtonText() const
{
	return bNewHyperlink ? LOCTEXT("HyperlinkInsertLabel", "Insert Hyperlink") : LOCTEXT("HyperlinkSetLabel", "Set Hyperlink");
}

void STutorialEditableText::OnActiveHyperlinkChanged(TSharedPtr<FHyperlinkTypeDesc> NewValue, ESelectInfo::Type SelectionType)
{
	CurrentHyperlinkType = NewValue;
}

TSharedRef<SWidget> STutorialEditableText::GenerateHyperlinkComboEntry(TSharedPtr<FHyperlinkTypeDesc> SourceEntry)
{
	return SNew(SBorder)
		.BorderImage( FAppStyle::Get().GetBrush( "NoBorder" ) )
		.ForegroundColor(FAppStyle::Get().GetSlateColor("InvertedForeground"))
		[
			SNew(STextBlock)
			.Text(SourceEntry->Text)
			.ToolTipText(SourceEntry->TooltipText)
			.TextStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>("TutorialEditableText.Toolbar.Text"))
		];
}

FText STutorialEditableText::GetActiveHyperlinkName() const
{
	if(CurrentHyperlinkType.IsValid())
	{
		return CurrentHyperlinkType->Text;
	}

	return FText();
}

FText STutorialEditableText::GetActiveHyperlinkTooltip() const
{
	if(CurrentHyperlinkType.IsValid())
	{
		return CurrentHyperlinkType->TooltipText;
	}

	return FText();
}

EVisibility STutorialEditableText::GetOpenAssetVisibility() const
{
	return CurrentHyperlinkType.IsValid() && CurrentHyperlinkType->Type == EHyperlinkType::Asset ? EVisibility::Visible : EVisibility::Collapsed;
}

void STutorialEditableText::HandleOpenAssetCheckStateChanged(ECheckBoxState InCheckState)
{
	bOpenAsset = !bOpenAsset;
}

ECheckBoxState STutorialEditableText::IsOpenAssetChecked() const
{
	return bOpenAsset ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility STutorialEditableText::GetExcerptVisibility() const
{
	return CurrentHyperlinkType.IsValid() && CurrentHyperlinkType->Type == EHyperlinkType::UDN ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply STutorialEditableText::HandleImageButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		TArray<FString> OutFiles;
		const FString Extension(TEXT("png"));
		const FString Filter = FString::Printf(TEXT("%s files (*.%s)|*.%s"), *Extension, *Extension, *Extension);
		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		if (DesktopPlatform->OpenFileDialog(ParentWindowHandle, FText::Format(LOCTEXT("ImagePickerDialogTitle", "Choose a {0} file"), FText::FromString(Extension)).ToString(), DefaultPath, TEXT(""), Filter, EFileDialogFlags::None, OutFiles))
		{
			check(OutFiles.Num() == 1);

			FRunInfo RunInfo(TEXT("img"));

			// the path to the image needs to be stored either as a 'long package name' version of itself
			// (minus png extension) or as a literal (base dir relative) path.
			FString ContentPath;
			if(FPackageName::TryConvertFilenameToLongPackageName(OutFiles[0], ContentPath))
			{
				RunInfo.MetaData.Add(TEXT("src"), ContentPath);
			}
			else
			{
				RunInfo.MetaData.Add(TEXT("src"), OutFiles[0]);
			}
			
			TSharedRef<FSlateImageRun> ImageRun = FSlateImageRun::Create(
				RunInfo, 
				MakeShareable(new FString(TEXT("\x200B"))), // Zero-Width Breaking Space
				FName(*FTutorialImageDecorator::GetPathToImage(OutFiles[0])),
				0
				);

			RichEditableTextBox->InsertRunAtCursor(ImageRun);

			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_IMPORT, FPaths::GetPath(OutFiles[0]));	
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
