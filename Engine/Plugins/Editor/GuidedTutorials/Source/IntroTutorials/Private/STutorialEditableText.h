// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Widgets/Input/SComboBox.h"

class SEditableTextBox;
class SMultiLineEditableTextBox;
struct FHyperlinkTypeDesc;
struct FTextStyleAndName;

class STutorialEditableText : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STutorialEditableText ){}

	SLATE_ATTRIBUTE(FText, Text)

	SLATE_ARGUMENT(FOnTextCommitted, OnTextCommitted)

	SLATE_ARGUMENT(FOnTextChanged, OnTextChanged)

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

	virtual bool SupportsKeyboardFocus() const override { return true; }

protected:
	
	TSharedPtr<const IRun> GetCurrentRun() const;

	void HandleRichEditableTextChanged(const FText& Text);

	void HandleRichEditableTextCommitted(const FText& Text, ETextCommit::Type Type);

	void HandleRichEditableTextCursorMoved(const FTextLocation& NewCursorPosition );

	FText GetActiveStyleName() const;

	void OnActiveStyleChanged(TSharedPtr<struct FTextStyleAndName> NewValue, ESelectInfo::Type);

	TSharedRef<SWidget> GenerateStyleComboEntry(TSharedPtr<struct FTextStyleAndName> SourceEntry);

	void StyleSelectedText();

	void HandleHyperlinkComboOpened();

	bool IsHyperlinkComboEnabled() const;

	FReply HandleInsertHyperLinkClicked();

	EVisibility GetToolbarVisibility() const;

	FText GetHyperlinkButtonText() const;

	void OnActiveHyperlinkChanged(TSharedPtr<struct FHyperlinkTypeDesc> NewValue, ESelectInfo::Type SelectionType);

	TSharedRef<SWidget> GenerateHyperlinkComboEntry(TSharedPtr<struct FHyperlinkTypeDesc> SourceEntry);

	FText GetActiveHyperlinkName() const;

	FText GetActiveHyperlinkTooltip() const;

	TSharedPtr<FHyperlinkTypeDesc> GetHyperlinkTypeFromId(const FString& InId) const;

	EVisibility GetOpenAssetVisibility() const;

	void HandleOpenAssetCheckStateChanged(ECheckBoxState InCheckState);

	ECheckBoxState IsOpenAssetChecked() const;

	EVisibility GetExcerptVisibility() const;

	FReply HandleImageButtonClicked();

protected:
	TSharedPtr<SMultiLineEditableTextBox> RichEditableTextBox;

	FSlateHyperlinkRun::FOnClick OnBrowserLinkClicked;
	FSlateHyperlinkRun::FOnClick OnDocLinkClicked;
	FSlateHyperlinkRun::FOnClick OnTutorialLinkClicked;
	FSlateHyperlinkRun::FOnClick OnCodeLinkClicked;
	FSlateHyperlinkRun::FOnClick OnAssetLinkClicked;

	TSharedPtr<SComboButton> HyperlinkComboButton;
	TSharedPtr<SComboBox<TSharedPtr<struct FTextStyleAndName>>> FontComboBox;
	TSharedPtr<STextBlock> HyperlinkNameTextBlock;
	TSharedPtr<SEditableTextBox> HyperlinkURLTextBox;
	TSharedPtr<SEditableTextBox> UDNExcerptTextBox;

	TSharedPtr<struct FTextStyleAndName> ActiveStyle;
	TSharedPtr<struct FTextStyleAndName> HyperlinkStyle;

	TArray<TSharedPtr<struct FTextStyleAndName>> StylesAndNames;

	FOnTextCommitted OnTextCommitted;
	FOnTextChanged OnTextChanged;

	TSharedPtr<struct FHyperlinkTypeDesc> CurrentHyperlinkType;

	bool bOpenAsset;

	bool bNewHyperlink;
};
