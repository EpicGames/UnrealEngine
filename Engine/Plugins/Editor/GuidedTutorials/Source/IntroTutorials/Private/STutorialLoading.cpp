// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialLoading.h"
#include "Fonts/SlateFontInfo.h"
#include "Misc/Paths.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Images/SThrobber.h"

#define LOCTEXT_NAMESPACE "Tutorials"

void STutorialLoading::Construct(const FArguments& InArgs)
{
	ContextWindow = InArgs._ContextWindow;

	ChildSlot
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SBorder)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Center)
				[
					SNew(SCircularThrobber)
				]
				+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.ShadowColorAndOpacity(FLinearColor::Black)
						.ColorAndOpacity(FLinearColor::White)
						.ShadowOffset(FIntPoint(-1, 1))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 16))
						.Text(LOCTEXT("LoadingContentTut", "Loading Tutorial Content"))
					]
			]
		];
}

#undef LOCTEXT_NAMESPACE
