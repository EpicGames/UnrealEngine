// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWindow;

class STutorialLoading : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STutorialLoading) {}
	
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ContextWindow)
	
	SLATE_END_ARGS()

public:

	/** Widget constructor */
	void Construct(const FArguments& InArgs);

private:
	/** Window where loading visual will be displayed */
	TWeakPtr<SWindow> ContextWindow;
};
