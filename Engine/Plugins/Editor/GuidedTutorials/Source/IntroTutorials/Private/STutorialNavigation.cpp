// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialNavigation.h"
#include "EditorTutorialStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"

void STutorialNavigation::Construct(const FArguments& InArgs)
{
	OnBackClicked = InArgs._OnBackClicked;
	OnHomeClicked = InArgs._OnHomeClicked;
	OnNextClicked = InArgs._OnNextClicked;
	IsBackEnabled = InArgs._IsBackEnabled;
	IsHomeEnabled = InArgs._IsHomeEnabled;
	IsNextEnabled = InArgs._IsNextEnabled;
	OnGetProgress = InArgs._OnGetProgress;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(24.0f)
		.BorderImage(FEditorTutorialStyle::Get().GetBrush("Tutorials.Border"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &STutorialNavigation::OnBackButtonClicked)
					.IsEnabled(InArgs._IsBackEnabled)
					.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.Button"))
					.Content()
					[
						SNew(SImage)
						.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Back"))
						.ColorAndOpacity(this, &STutorialNavigation::GetBackButtonColor)
					]
				]
				+SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &STutorialNavigation::OnHomeButtonClicked)
					.IsEnabled(InArgs._IsHomeEnabled)
					.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.Button"))
					.Content()
					[
						SNew(SImage)
						.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Home"))
						.ColorAndOpacity(this, &STutorialNavigation::GetHomeButtonColor)
					]
				]
				+SHorizontalBox::Slot()
				.Padding(2.0f)
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(this, &STutorialNavigation::OnNextButtonClicked)
					.IsEnabled(InArgs._IsNextEnabled)
					.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Content.Button"))
					.Content()
					[
						SNew(SImage)
						.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Next"))
						.ColorAndOpacity(this, &STutorialNavigation::GetNextButtonColor)
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(4.0f)
			.Padding(0.0f, 2.0f)
			[
				SNew(SProgressBar)
				.Percent(this, &STutorialNavigation::GetPercentComplete)
			]
		]
	];
}

FReply STutorialNavigation::OnBackButtonClicked()
{
	OnBackClicked.ExecuteIfBound();
	return FReply::Handled();
}

FSlateColor STutorialNavigation::GetBackButtonColor() const
{
	return IsBackEnabled.Get() ? FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) : FLinearColor(1.0f, 1.0f, 1.0f, 0.25f);
}

FReply STutorialNavigation::OnHomeButtonClicked()
{
	OnHomeClicked.ExecuteIfBound();
	return FReply::Handled();
}

FSlateColor STutorialNavigation::GetHomeButtonColor() const
{
	return IsHomeEnabled.Get() ? FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) : FLinearColor(1.0f, 1.0f, 1.0f, 0.25f);
}

FReply STutorialNavigation::OnNextButtonClicked()
{
	OnNextClicked.ExecuteIfBound();
	return FReply::Handled();
}

FSlateColor STutorialNavigation::GetNextButtonColor() const
{
	return IsNextEnabled.Get() ? FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) : FLinearColor(1.0f, 1.0f, 1.0f, 0.25f);
}

TOptional<float> STutorialNavigation::GetPercentComplete() const
{
	return OnGetProgress.Get();
}
