// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWindow;

/**
 * The widget which displays floating navigation controls
 */
class STutorialNavigation : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STutorialNavigation) {}

	SLATE_ARGUMENT(FSimpleDelegate, OnBackClicked)
	SLATE_ARGUMENT(FSimpleDelegate, OnHomeClicked)
	SLATE_ARGUMENT(FSimpleDelegate, OnNextClicked)
	SLATE_ATTRIBUTE(bool, IsBackEnabled)
	SLATE_ATTRIBUTE(bool, IsHomeEnabled)
	SLATE_ATTRIBUTE(bool, IsNextEnabled)
	SLATE_ATTRIBUTE(float, OnGetProgress)
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	/** Handle back button being clicked */
	FReply OnBackButtonClicked();

	/** Handle back button color - used to dim the button when disabled */
	FSlateColor GetBackButtonColor() const;

	/** Handle home button being clicked */
	FReply OnHomeButtonClicked();

	/** Handle home button color - used to dim the button when disabled */
	FSlateColor GetHomeButtonColor() const;

	/** Handle next button being clicked */
	FReply OnNextButtonClicked();

	/** Handle next button color - used to dim the button when disabled */
	FSlateColor GetNextButtonColor() const;

	/** Handle display of the progress bar */
	TOptional<float> GetPercentComplete() const;

private:

	FSimpleDelegate OnBackClicked;
	FSimpleDelegate OnHomeClicked;
	FSimpleDelegate OnNextClicked;
	TAttribute<bool> IsBackEnabled;
	TAttribute<bool> IsHomeEnabled;
	TAttribute<bool> IsNextEnabled;
	TAttribute<float> OnGetProgress;
};
