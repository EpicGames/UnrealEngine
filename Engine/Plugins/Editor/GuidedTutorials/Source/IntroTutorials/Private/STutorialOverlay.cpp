// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialOverlay.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "Misc/PackageName.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SCanvas.h"
#include "Framework/Docking/TabManager.h"
#include "Engine/Blueprint.h"
#include "Editor.h"

#include "IntroTutorials.h"
#include "STutorialContent.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "LevelEditor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"

static FName IntroTutorialsModuleName("IntroTutorials");

void STutorialOverlay::Construct(const FArguments& InArgs, UEditorTutorial* InTutorial, FTutorialStage* const InStage)
{
	ParentWindow = InArgs._ParentWindow;
	bIsStandalone = InArgs._IsStandalone;
	OnClosed = InArgs._OnClosed;
	bHasValidContent = InStage != nullptr;
	OnWidgetWasDrawn = InArgs._OnWidgetWasDrawn;

	TSharedPtr<SOverlay> Overlay;

	ChildSlot
	[
		SAssignNew(Overlay, SOverlay)
		+SOverlay::Slot()
		[
			SAssignNew(OverlayCanvas, SCanvas)
		]
	];

	if(InStage != nullptr)
	{
		// add non-widget content, if any
		if(InArgs._AllowNonWidgetContent && InStage->Content.Type != ETutorialContent::None)
		{
			Overlay->AddSlot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STutorialContent, InTutorial, InStage->Content)
					.OnClosed(InArgs._OnClosed)
					.OnNextClicked(InArgs._OnNextClicked)
					.OnHomeClicked(InArgs._OnHomeClicked)
					.OnBackClicked(InArgs._OnBackClicked)
					.IsBackEnabled(InArgs._IsBackEnabled)
					.IsHomeEnabled(InArgs._IsHomeEnabled)
					.IsNextEnabled(InArgs._IsNextEnabled)
					.IsStandalone(InArgs._IsStandalone)
					.WrapTextAt(600.0f)
					.NextButtonText(InStage->NextButtonText)
					.BackButtonText(InStage->BackButtonText)
				]
			];
		}

		if(InStage->WidgetContent.Num() > 0)
		{
			FIntroTutorials& IntroTutorials = FModuleManager::Get().GetModuleChecked<FIntroTutorials>("IntroTutorials");

			// now add canvas slots for widget-bound content
			for (const FTutorialWidgetContent& WidgetContent : InStage->WidgetContent)
			{
				if (WidgetContent.Content.Type != ETutorialContent::None)
				{
					TSharedPtr<STutorialContent> ContentWidget =
						SNew(STutorialContent, InTutorial, WidgetContent.Content)
						.HAlign(WidgetContent.HorizontalAlignment)
						.VAlign(WidgetContent.VerticalAlignment)
						.Offset(WidgetContent.Offset)
						.IsStandalone(bIsStandalone)
						.OnClosed(InArgs._OnClosed)
						.OnNextClicked(InArgs._OnNextClicked)
						.OnHomeClicked(InArgs._OnHomeClicked)
						.OnBackClicked(InArgs._OnBackClicked)
						.IsBackEnabled(InArgs._IsBackEnabled)
						.IsHomeEnabled(InArgs._IsHomeEnabled)
						.IsNextEnabled(InArgs._IsNextEnabled)
						.WrapTextAt(WidgetContent.ContentWidth)
						.Anchor(WidgetContent.WidgetAnchor)
						.AllowNonWidgetContent(InArgs._AllowNonWidgetContent)
						.OnWasWidgetDrawn(InArgs._OnWasWidgetDrawn);
					
					PerformWidgetInteractions(InTutorial, WidgetContent); 					

					OverlayCanvas->AddSlot()
						.Position(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(ContentWidget.Get(), &STutorialContent::GetPosition)))
						.Size(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(ContentWidget.Get(), &STutorialContent::GetSize)))
						[
							ContentWidget.ToSharedRef()
						];

					OnPaintNamedWidget.AddSP(ContentWidget.Get(), &STutorialContent::HandlePaintNamedWidget);
					OnResetNamedWidget.AddSP(ContentWidget.Get(), &STutorialContent::HandleResetNamedWidget);
					OnCacheWindowSize.AddSP(ContentWidget.Get(), &STutorialContent::HandleCacheWindowSize);
				}
			}
		}
	}
}

int32 STutorialOverlay::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	if(ParentWindow.IsValid())
	{
		bool bIsPicking = false;
		FName WidgetNameToHighlight = NAME_None;
		FIntroTutorials& IntroTutorials = FModuleManager::Get().GetModuleChecked<FIntroTutorials>(IntroTutorialsModuleName);
		if(IntroTutorials.OnIsPicking().IsBound())
		{
			bIsPicking = IntroTutorials.OnIsPicking().Execute(WidgetNameToHighlight);
		}

		if(bIsPicking || bHasValidContent)
		{
			TSharedPtr<SWindow> PinnedWindow = ParentWindow.Pin();
			OnResetNamedWidget.Broadcast();
			OnCacheWindowSize.Broadcast(PinnedWindow->GetWindowGeometryInWindow().GetLocalSize());
			LayerId = TraverseWidgets(PinnedWindow.ToSharedRef(), PinnedWindow->GetWindowGeometryInWindow(), MyCullingRect, OutDrawElements, LayerId);
		}
	}
	
	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

int32 STutorialOverlay::TraverseWidgets(TSharedRef<SWidget> InWidget, const FGeometry& InGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	bool bIsPicking = false;
	bool bShouldHighlight = false;
	bool bShouldDraw = false;
	FName WidgetNameToHighlight = NAME_None;
	FIntroTutorials& IntroTutorials = FModuleManager::Get().GetModuleChecked<FIntroTutorials>(IntroTutorialsModuleName);
	if (IntroTutorials.OnValidatePickingCandidate().IsBound())
	{
		bIsPicking = IntroTutorials.OnValidatePickingCandidate().Execute(InWidget,WidgetNameToHighlight,bShouldHighlight);
	}

	// First draw the widget if we should
	TSharedPtr<FTagMetaData> WidgetMetaData = InWidget->GetMetaData<FTagMetaData>();
	const FName WidgetTag = (WidgetMetaData.IsValid() && WidgetMetaData->Tag.IsValid()) ? WidgetMetaData->Tag : InWidget->GetTag();
	if (WidgetTag != NAME_None || WidgetMetaData.IsValid())
	{
		// we are a named widget - ask it to draw
		OnPaintNamedWidget.Broadcast(InWidget, InGeometry);
		OnWidgetWasDrawn.ExecuteIfBound(WidgetTag);
	}

	// Next check and draw the highlight as appropriate
	if (bIsPicking == true)
	{
		// if we are picking, we need to draw an outline here	
		if(WidgetNameToHighlight != NAME_None )
		{
			const FLinearColor Color = bIsPicking && bShouldHighlight ? FLinearColor::Green : FLinearColor::White;
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, InGeometry.ToPaintGeometry(), FAppStyle::Get().GetBrush(TEXT("Debug.Border")), ESlateDrawEffect::None, Color);
		}	
	}

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	InWidget->ArrangeChildren(InGeometry, ArrangedChildren);
	for(int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ChildIndex++)
	{
		const FArrangedWidget& ArrangedWidget = ArrangedChildren[ChildIndex];
		LayerId = TraverseWidgets(ArrangedWidget.Widget, ArrangedWidget.Geometry, MyCullingRect, OutDrawElements, LayerId);
	}

	return LayerId;
}

void STutorialOverlay::PerformWidgetInteractions(UEditorTutorial* InTutorial, const FTutorialWidgetContent &WidgetContent)
{
	// Open any browser we need too
	OpenBrowserForWidgetAnchor(InTutorial, WidgetContent);

	FocusOnAnyBlueprintNodes(WidgetContent);
}

void STutorialOverlay::OpenBrowserForWidgetAnchor(UEditorTutorial* InTutorial, const FTutorialWidgetContent &WidgetContent)
{
	if(!WidgetContent.WidgetAnchor.TabToFocusOrOpen.IsEmpty())
	{
		IAssetEditorInstance* AssetEditor = nullptr;

		// Check to see if we can find a blueprint relevant to this node and open the editor for that (Then try to get the tabmanager from that)
		if(WidgetContent.WidgetAnchor.OuterName.Len() > 0)
		{
			// Remove the prefix from the name
			int32 Space = WidgetContent.WidgetAnchor.OuterName.Find(TEXT(" "));
			FString Name = WidgetContent.WidgetAnchor.OuterName.RightChop(Space + 1);
			TArray<FString> AssetPaths;
			AssetPaths.Add(Name);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(AssetPaths);
			UObject* Blueprint = FindFirstObject<UObject>(*Name, EFindFirstObjectOptions::EnsureIfAmbiguous);

			// If we found a blueprint
			if(Blueprint != nullptr)
			{
				IAssetEditorInstance* PotentialAssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Blueprint, false);
				if(PotentialAssetEditor != nullptr)
				{
					AssetEditor = PotentialAssetEditor;
				}
			}
		}

		// If we haven't found a tab manager, next check the asset editor that we reference in this tutorial, if any
		if(AssetEditor == nullptr)
		{
			// Try looking for the object that this tutorial references (it should already be loaded by this tutorial if it exists).
			UObject* AssetObject = InTutorial->AssetToUse.ResolveObject();
			if(AssetObject != nullptr)
			{
				IAssetEditorInstance* PotentialAssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AssetObject, false);
				if(PotentialAssetEditor != nullptr)
				{
					AssetEditor = PotentialAssetEditor;
				}
			}
		}

		// Invoke any tab
		if(AssetEditor != nullptr)
		{
			AssetEditor->InvokeTab(FTabId(*WidgetContent.WidgetAnchor.TabToFocusOrOpen));
		}
		else
		{
			// fallback to trying the main level editor tab manager
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
			TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
			LevelEditorTabManager->TryInvokeTab(FName(*WidgetContent.WidgetAnchor.TabToFocusOrOpen));
		}
	}
}

void STutorialOverlay::FocusOnAnyBlueprintNodes(const FTutorialWidgetContent &WidgetContent)
{
	if (WidgetContent.bAutoFocus == false)
	{
		return;
	}
	
	const FString Name = WidgetContent.WidgetAnchor.OuterName;
	const FString ObjectPath = WidgetContent.WidgetAnchor.WrapperIdentifier.ToString();
	int32 NameIndex;
	Name.FindLastChar(TEXT('.'), NameIndex);
	FString BlueprintName = Name.RightChop(NameIndex + 1);
	UBlueprint* Blueprint = FindFirstObject<UBlueprint>(*BlueprintName, EFindFirstObjectOptions::EnsureIfAmbiguous);
	// If we find a blueprint
	if (Blueprint != nullptr)
	{
		// Try to grab guid
		FGuid NodeGuid;
		FGuid::Parse(WidgetContent.WidgetAnchor.GUIDString, NodeGuid);
		UEdGraphNode* OutNode = NULL;
		if (UEdGraphNode* GraphNode = FBlueprintEditorUtils::GetNodeByGUID(Blueprint, NodeGuid))
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(GraphNode, false);
		}
	}
	else if ( !ObjectPath.IsEmpty() )
	{
		// if we didn't have a blueprint object to focus on, try it with a regular one
		UE_CLOGF(FPackageName::IsShortPackageName(ObjectPath), LogClass, Warning, "FocusOnAnyBlueprintNodes expects path names for WrapperIdentifier, got short name instead: \"%ls\"", *ObjectPath);
		UObject* FocusObject = FindObject<UObject>(nullptr, *ObjectPath);
		// If we didn't find it, maybe it just hasn't been loaded yet
		if( FocusObject == nullptr )
		{
			FocusObject = LoadObject<UObject>(nullptr, *ObjectPath, nullptr, LOAD_FindIfFail);
		}
		// If we found an asset redirector, we need to follow it
		UObjectRedirector* Redir = dynamic_cast<UObjectRedirector*>(FocusObject);
		if (Redir)
		{
			FocusObject = Redir->DestinationObject;
		}

		// If we failed to find the object, it may be a class that has been redirected
		if (!FocusObject)
		{
			const FString RedirectedObjectName = FLinkerLoad::FindNewPathNameForClass(ObjectPath, false);
			if (!RedirectedObjectName.IsEmpty())
			{
				FocusObject = UClass::TryFindTypeSlow<UClass>(RedirectedObjectName);
			}
		}

		if (FocusObject)
		{
			TArray< UObject* > Objects;
			Objects.Add(FocusObject);
			GEditor->SyncBrowserToObjects(Objects);
		}
	}
}

