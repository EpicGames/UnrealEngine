// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class SCanvas;
class SWindow;
class UEditorTutorial;
struct FTutorialStage;
struct FTutorialWidgetContent;

/** Delegate used when drawing/arranging widgets */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPaintNamedWidget, TSharedRef<SWidget> /*InWidget*/, const FGeometry& /*InGeometry*/);

/** Delegate used to inform widgets of the current window size, so they can auto-adjust layout */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCacheWindowSize, const FVector2D& /*InWindowSize*/);

/** Delegates for registering & querying whether a widget was drawn */
DECLARE_DELEGATE_OneParam(FOnWidgetWasDrawn, const FName& /*InName*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnWasWidgetDrawn, const FName& /*InName*/);

/**
 * The widget which displays multiple 'floating' pieces of content overlaid onto the editor
 */
class STutorialOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STutorialOverlay ) 
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}

	/** The window this content is displayed over */
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)

	/** Whether this a standalone widget (with its own close button) or part of a group of other widgets, paired with tutorial navigation */
	SLATE_ARGUMENT(bool, IsStandalone)

	/** Delegate fired when the close button is clicked */
	SLATE_EVENT(FSimpleDelegate, OnClosed)

	SLATE_EVENT(FSimpleDelegate, OnBackClicked)
	SLATE_EVENT(FSimpleDelegate, OnHomeClicked)
	SLATE_EVENT(FSimpleDelegate, OnNextClicked)
	SLATE_ATTRIBUTE(bool, IsBackEnabled)
	SLATE_ATTRIBUTE(bool, IsHomeEnabled)
	SLATE_ATTRIBUTE(bool, IsNextEnabled)

	/** Whether we can show full window content in this overlay (i.e. in the same window as the navigation controls) */
	SLATE_ARGUMENT(bool, AllowNonWidgetContent)

	/** Delegates for registering & querying whether a widget was drawn */
	SLATE_EVENT(FOnWidgetWasDrawn, OnWidgetWasDrawn)
	SLATE_EVENT(FOnWasWidgetDrawn, OnWasWidgetDrawn)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEditorTutorial* InTutorial, FTutorialStage* const InStage);

	/** SWidget implementation */
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

private:
	/** Recursive function used to re-generate widget geometry and forward the geometry of named widgets onto their respective content */
	int32 TraverseWidgets(TSharedRef<SWidget> InWidget, const FGeometry& InGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	/* Opens the browser that the given widget requires if it is not already. */
	void OpenBrowserForWidgetAnchor(UEditorTutorial* InTutorial, const FTutorialWidgetContent &WidgetContent);

	/* Focus on the blueprint node if an anchor widget references one */
	void FocusOnAnyBlueprintNodes(const FTutorialWidgetContent &WidgetContent);

	/* Do any interaction stuff for a widget - open browser, scroll to node etc */
	void PerformWidgetInteractions(UEditorTutorial* InTutorial, const FTutorialWidgetContent &WidgetContent);

private:
	/** Reference to the canvas we use to position our content widgets */
	TSharedPtr<SCanvas> OverlayCanvas;

	/** The window this content is displayed over */
	TWeakPtr<SWindow> ParentWindow;

	/** Whether this a standalone widget (with its own close button) or part of a group of other widgets, paired with tutorial navigation */
	bool bIsStandalone;

	/** Delegate fired when a cos ebutton is clicked in tutorial content */
	FSimpleDelegate OnClosed;

	/** Delegate used when drawing/arranging widgets */
	FOnPaintNamedWidget OnPaintNamedWidget;

	/** Delegate used to reset drawing of named widgets */
	FSimpleMulticastDelegate OnResetNamedWidget;

	/** Delegate used to inform widgets of the current window size, so they can auto-adjust layout */
	FOnCacheWindowSize OnCacheWindowSize;

	/** Flag to see if we have valid content (this widget is created to also supply picker overlays) */
	bool bHasValidContent;

	/** Delegate for querying whether a widget was drawn */
	FOnWidgetWasDrawn OnWidgetWasDrawn;
};
