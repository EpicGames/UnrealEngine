// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialRoot.h"
#include "Framework/Application/SlateApplication.h"
#include "Templates/SubclassOf.h"
#include "Kismet/GameplayStatics.h"

#include "IntroTutorials.h"
#include "EditorTutorial.h"
#include "SEditorTutorials.h"
#include "EditorTutorialSettings.h"
#include "TutorialStateSettings.h"
#include "Toolkits/ToolkitManager.h"
#include "Toolkits/IToolkitHost.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "STutorialRoot"

void STutorialRoot::Construct(const FArguments& InArgs)
{
	CurrentTutorial = nullptr;
	CurrentTutorialStage = 0;
	CurrentTutorialStartTime = FPlatformTime::Seconds();

	ChildSlot
	[
		SNullWidget::NullWidget
	];
}

void STutorialRoot::AttachWidget(TSharedPtr<SWidget> Widget)
{
	//This checkSlow is just here to ensure we know what we're doing
	checkSlow(ChildSlot.GetWidget() == SNullWidget::NullWidget);
	ChildSlot.AttachWidget(Widget.ToSharedRef());
}

void STutorialRoot::DetachWidget()
{
	ChildSlot.DetachWidget();
}

void STutorialRoot::MaybeAddOverlay(TSharedRef<SWindow> InWindow)
{
	if(InWindow->HasOverlay())
	{
		// check we don't already have a widget overlay for this window
		TWeakPtr<SEditorTutorials>* FoundWidget = TutorialWidgets.Find(InWindow);
		if(FoundWidget == nullptr)
		{
			TSharedPtr<SEditorTutorials> TutorialWidget = nullptr;
			InWindow->AddOverlaySlot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(TutorialWidget, SEditorTutorials)
					.ParentWindow(InWindow)
					.OnNextClicked(FOnNextClicked::CreateSP(this, &STutorialRoot::HandleNextClicked))
					.OnBackClicked(FSimpleDelegate::CreateSP(this, &STutorialRoot::HandleBackClicked))
					.OnHomeClicked(FSimpleDelegate::CreateSP(this, &STutorialRoot::HandleHomeClicked))
					.OnCloseClicked(FSimpleDelegate::CreateSP(this, &STutorialRoot::HandleCloseClicked))
					.OnGetCurrentTutorial(FOnGetCurrentTutorial::CreateSP(this, &STutorialRoot::HandleGetCurrentTutorial))
					.OnGetCurrentTutorialStage(FOnGetCurrentTutorialStage::CreateSP(this, &STutorialRoot::HandleGetCurrentTutorialStage))
					.OnLaunchTutorial(FOnLaunchTutorial::CreateSP(this, &STutorialRoot::LaunchTutorial))
					.OnWasWidgetDrawn(FOnWasWidgetDrawn::CreateSP(this, &STutorialRoot::WasWidgetDrawn))
					.OnWidgetWasDrawn(FOnWidgetWasDrawn::CreateSP(this, &STutorialRoot::WidgetWasDrawn))
				]
			];

			FoundWidget = &TutorialWidgets.Add(InWindow, TutorialWidget);

			FoundWidget->Pin()->RebuildCurrentContent();
		}
	}

	TArray<TSharedRef<SWindow>> ChildWindows = InWindow->GetChildWindows();
	for(auto& ChildWindow : ChildWindows)
	{
		MaybeAddOverlay(ChildWindow);
	}
}

void STutorialRoot::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	TArray<TSharedRef<SWindow>> Windows = FSlateApplication::Get().GetInteractiveTopLevelWindows();
	for(auto& Window : Windows)
	{
		MaybeAddOverlay(Window);
	}

	// empty array but leave us the slack (we dont want to reallocate all the time, and this array should never grow too large)
	PreviouslyDrawnWidgets.Empty(PreviouslyDrawnWidgets.Max());
	PreviouslyDrawnWidgets.Append(MoveTemp(DrawnWidgets));
	DrawnWidgets.Empty(DrawnWidgets.Max());
}

void STutorialRoot::LaunchTutorial(UEditorTutorial* InTutorial, IIntroTutorials::ETutorialStartType InStartType, TWeakPtr<SWindow> InNavigationWindow, FSimpleDelegate InOnTutorialClosed, FSimpleDelegate InOnTutorialExited)
{
	if(InTutorial != nullptr)
	{
		CurrentTutorial = InTutorial;

		// we force a restart if this tutorial was completed
		if (GetDefault<UTutorialStateSettings>()->HaveCompletedTutorial(CurrentTutorial) && (InStartType == IIntroTutorials::ETutorialStartType::TST_CONTINUE))
		{
			InStartType = IIntroTutorials::ETutorialStartType::TST_RESTART;
		}

		bool bHaveSeenTutorial = false;
		switch (InStartType)
		{
		case IIntroTutorials::ETutorialStartType::TST_RESTART:
			CurrentTutorialStage = 0;
			break;
		case IIntroTutorials::ETutorialStartType::TST_LASTSTAGE:
			CurrentTutorialStage = FMath::Max(0, (CurrentTutorial->Stages.Num() - 1));
			break;
		default:
		case IIntroTutorials::ETutorialStartType::TST_CONTINUE:
			CurrentTutorialStage = GetDefault<UTutorialStateSettings>()->GetProgress(CurrentTutorial, bHaveSeenTutorial);
			break;
		}

		// check if we should be launching this tutorial for an asset editor
		if(InTutorial->AssetToUse.IsValid())
		{
			TArray<FString> AssetPaths;
			AssetPaths.Add(InTutorial->AssetToUse.ToString());
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(AssetPaths);

			UObject* Asset = InTutorial->AssetToUse.ResolveObject();
			if(Asset != nullptr)
			{
				TSharedPtr<IToolkit> Toolkit = FToolkitManager::Get().FindEditorForAsset( Asset );
				if(Toolkit.IsValid())
				{
					InNavigationWindow = FSlateApplication::Get().FindWidgetWindow(Toolkit->GetToolkitHost()->GetParentWidget());

					// make sure we have a valid tutorial overlay
					if(InNavigationWindow.IsValid())
					{
						MaybeAddOverlay(InNavigationWindow.Pin().ToSharedRef());
					}
				}
			}
		}

		CurrentTutorialStartTime = FPlatformTime::Seconds();

		// launch tutorial for all windows we wrap - any tutorial can display over any window
		for(auto& TutorialWidget : TutorialWidgets)
		{
			if(TutorialWidget.Value.IsValid())
			{
				bool bIsNavigationWindow = false;
				if (!InNavigationWindow.IsValid())
				{
					bIsNavigationWindow = TutorialWidget.Value.Pin()->IsNavigationVisible();
				}
				else
				{
					bIsNavigationWindow = (TutorialWidget.Value.Pin()->GetParentWindow() == InNavigationWindow.Pin());
				}
				TutorialWidget.Value.Pin()->LaunchTutorial(bIsNavigationWindow, InOnTutorialClosed, InOnTutorialExited);
			}
		}

		if (CurrentTutorial != nullptr)
		{
			CurrentTutorial->HandleTutorialLaunched();
		}

		if (CurrentTutorial != nullptr && CurrentTutorialStage < CurrentTutorial->Stages.Num())
		{
			CurrentTutorial->HandleTutorialStageStarted(CurrentTutorial->Stages[CurrentTutorialStage].Name);
		}
	}
}

void STutorialRoot::CloseAllTutorialContent()
{
	for (auto& TutorialWidget : TutorialWidgets)
	{
		if (TutorialWidget.Value.IsValid())
		{
			TutorialWidget.Value.Pin()->HideContent();
		}
	}
}

void STutorialRoot::HandleNextClicked(TWeakPtr<SWindow> InNavigationWindow)
{
	GoToNextStage(InNavigationWindow);
}

void STutorialRoot::HandleBackClicked()
{
	if( FEngineAnalytics::IsAvailable() && CurrentTutorial != nullptr)
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Context.Tutorial"), FIntroTutorials::AnalyticsEventNameFromTutorial(CurrentTutorial)));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Context.StageIndex"), CurrentTutorialStage));

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.ClickedBackButton"), EventAttributes );
	}

	GoToPreviousStage();

	for(auto& TutorialWidget : TutorialWidgets)
	{
		if(TutorialWidget.Value.IsValid())
		{
			TSharedPtr<SEditorTutorials> PinnedTutorialWidget = TutorialWidget.Value.Pin();
			PinnedTutorialWidget->RebuildCurrentContent();
		}
	}
}

void STutorialRoot::HandleHomeClicked()
{
	if(CurrentTutorial != nullptr)
	{
		CurrentTutorial->HandleTutorialClosed();
		GetMutableDefault<UTutorialStateSettings>()->RecordProgress(CurrentTutorial, CurrentTutorialStage);
		GetMutableDefault<UTutorialStateSettings>()->SaveProgress();
	}

	// submit analytics data
	if( FEngineAnalytics::IsAvailable() && CurrentTutorial != nullptr && CurrentTutorialStage < CurrentTutorial->Stages.Num() )
	{
		FString const CurrentExcerptTitle = CurrentTutorial->Stages[CurrentTutorialStage].Name.ToString();
		int32 const CurrentExcerptIndex = CurrentTutorialStage;
		float const CurrentPageElapsedTime = (float)(FPlatformTime::Seconds() - CurrentTutorialStartTime);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LastStageIndex"), CurrentExcerptIndex));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LastStageTitle"), CurrentExcerptTitle));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TimeSpentInTutorial"), CurrentPageElapsedTime));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TutorialAsset"), FIntroTutorials::AnalyticsEventNameFromTutorial(CurrentTutorial)));
			
		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.Home"), EventAttributes );
	}

	CurrentTutorial = nullptr;
	CurrentTutorialStage = 0;

	for(auto& TutorialWidget : TutorialWidgets)
	{
		if(TutorialWidget.Value.IsValid())
		{
			TSharedPtr<SEditorTutorials> PinnedTutorialWidget = TutorialWidget.Value.Pin();
			PinnedTutorialWidget->RebuildCurrentContent();
		}
	}
}

UEditorTutorial* STutorialRoot::HandleGetCurrentTutorial()
{
	return CurrentTutorial;
}

int32 STutorialRoot::HandleGetCurrentTutorialStage()
{
	return CurrentTutorialStage;
}

void STutorialRoot::AddReferencedObjects( FReferenceCollector& Collector )
{
	if(CurrentTutorial != nullptr)
	{
		Collector.AddReferencedObject(CurrentTutorial);
	}
}

void STutorialRoot::GoToPreviousStage()
{
	if (CurrentTutorial != nullptr)
	{
		UEditorTutorial* OldTutorial = CurrentTutorial;
		int32 OldTutorialStage = CurrentTutorialStage;
		if (CurrentTutorialStage < CurrentTutorial->Stages.Num())
		{
			CurrentTutorial->HandleTutorialStageEnded(CurrentTutorial->Stages[CurrentTutorialStage].Name);
		}
		
		FString PlatformName = UGameplayStatics::GetPlatformName();
		for (--CurrentTutorialStage; CurrentTutorialStage >= 0; --CurrentTutorialStage)
		{
			bool bPlatformFoundInList = false;
			for (FString PlatformToTest : CurrentTutorial->Stages[CurrentTutorialStage].PlatformsToTest)
			{
				if (!PlatformName.Compare(PlatformToTest, ESearchCase::IgnoreCase))
				{
					bPlatformFoundInList = true;
					break;
				}
			}
			if (bPlatformFoundInList != CurrentTutorial->Stages[CurrentTutorialStage].bInvertPlatformTest)
			{
				// Skip this stage
				continue;
			}
			// We hit a page that we don't want to skip. Record progress and stop looking.
			GetMutableDefault<UTutorialStateSettings>()->RecordProgress(CurrentTutorial, CurrentTutorialStage);
			break;
		}
		if (CurrentTutorialStage < 0)
		{
			// We went out of bounds for this tutorial, so see if we want to go to another one.
			CurrentTutorialStage = 0;
			if (FName(*CurrentTutorial->PreviousTutorial.ToString()) != NAME_None)
			{
				TSubclassOf<UEditorTutorial> PreviousTutorialClass = LoadClass<UEditorTutorial>(NULL, *CurrentTutorial->PreviousTutorial.ToString(), NULL, LOAD_None, NULL);
				if (PreviousTutorialClass != nullptr)
				{
					LaunchTutorial(PreviousTutorialClass->GetDefaultObject<UEditorTutorial>(), IIntroTutorials::ETutorialStartType::TST_LASTSTAGE, nullptr, FSimpleDelegate(), FSimpleDelegate());
				}
				else
				{
					FSlateNotificationManager::Get().AddNotification(FNotificationInfo(FText::Format(LOCTEXT("PreviousTutorialNotFound", "Could not start previous tutorial {0}"), FText::FromString(CurrentTutorial->PreviousTutorial.ToString()))));
				}
			}
		}

		if (CurrentTutorial != nullptr && CurrentTutorialStage < CurrentTutorial->Stages.Num() && (CurrentTutorial != OldTutorial || CurrentTutorialStage != OldTutorialStage))
		{
			CurrentTutorial->HandleTutorialStageStarted(CurrentTutorial->Stages[CurrentTutorialStage].Name);
		}
	}

	for (auto& TutorialWidget : TutorialWidgets)
	{
		if (TutorialWidget.Value.IsValid())
		{
			TSharedPtr<SEditorTutorials> PinnedTutorialWidget = TutorialWidget.Value.Pin();
			PinnedTutorialWidget->RebuildCurrentContent();
		}
	}
}

void STutorialRoot::GoToNextStage(TWeakPtr<SWindow> InNavigationWindow)
{
	if(CurrentTutorial != nullptr)
	{
		UEditorTutorial* OldTutorial = CurrentTutorial;
		int32 OldTutorialStage = CurrentTutorialStage;
		if(CurrentTutorialStage < CurrentTutorial->Stages.Num())
		{
			CurrentTutorial->HandleTutorialStageEnded(CurrentTutorial->Stages[CurrentTutorialStage].Name);
		}

		FString PlatformName = UGameplayStatics::GetPlatformName();
		for (++CurrentTutorialStage; CurrentTutorialStage < CurrentTutorial->Stages.Num(); ++CurrentTutorialStage)
		{
			bool bPlatformFoundInList = false;
			for (FString PlatformToTest : CurrentTutorial->Stages[CurrentTutorialStage].PlatformsToTest)
			{
				if (!PlatformName.Compare(PlatformToTest, ESearchCase::IgnoreCase))
				{
					bPlatformFoundInList = true;
					break;
				}
			}
			if (bPlatformFoundInList != CurrentTutorial->Stages[CurrentTutorialStage].bInvertPlatformTest)
			{
				// Skip this stage
				continue;
			}
			// We hit a page that we don't want to skip. Record progress and stop looking.
			GetMutableDefault<UTutorialStateSettings>()->RecordProgress(CurrentTutorial, CurrentTutorialStage);
			break;
		}
		if (CurrentTutorialStage >= CurrentTutorial->Stages.Num())
		{
			// We went out of bounds for this tutorial, so see if we want to go to another one.
			CurrentTutorialStage = CurrentTutorial->Stages.Num() - 1;
			if (FName(*CurrentTutorial->NextTutorial.ToString()) != NAME_None)
			{
				TSubclassOf<UEditorTutorial> NextTutorialClass = LoadClass<UEditorTutorial>(NULL, *CurrentTutorial->NextTutorial.ToString(), NULL, LOAD_None, NULL);
				if (NextTutorialClass != nullptr)
				{
					LaunchTutorial(NextTutorialClass->GetDefaultObject<UEditorTutorial>(), IIntroTutorials::ETutorialStartType::TST_RESTART, InNavigationWindow, FSimpleDelegate(), FSimpleDelegate());
				}
				else
				{
					FSlateNotificationManager::Get().AddNotification(FNotificationInfo(FText::Format(LOCTEXT("NextTutorialNotFound", "Could not start next tutorial {0}"), FText::FromString(CurrentTutorial->NextTutorial.ToString()))));
				}
			}
		}

		if (CurrentTutorial != nullptr && CurrentTutorialStage < CurrentTutorial->Stages.Num() && (CurrentTutorial != OldTutorial || CurrentTutorialStage != OldTutorialStage))
		{
			CurrentTutorial->HandleTutorialStageStarted(CurrentTutorial->Stages[CurrentTutorialStage].Name);
		}
	}

	for(auto& TutorialWidget : TutorialWidgets)
	{
		if(TutorialWidget.Value.IsValid())
		{
			TSharedPtr<SEditorTutorials> PinnedTutorialWidget = TutorialWidget.Value.Pin();
			PinnedTutorialWidget->RebuildCurrentContent();
		}
	}
}

void STutorialRoot::HandleCloseClicked()
{
	if(CurrentTutorial != nullptr)
	{
		CurrentTutorial->HandleTutorialClosed();
		// Update the current stage when we close
		bool bHaveSeenTutorial = false;
		CurrentTutorialStage = GetDefault<UTutorialStateSettings>()->GetProgress(CurrentTutorial, bHaveSeenTutorial);
		GetMutableDefault<UTutorialStateSettings>()->RecordProgress(CurrentTutorial, CurrentTutorialStage);
		GetMutableDefault<UTutorialStateSettings>()->SaveProgress();
	}

	// submit analytics data
	if( FEngineAnalytics::IsAvailable() && CurrentTutorial != nullptr && CurrentTutorialStage < CurrentTutorial->Stages.Num() )
	{
		UEditorTutorial* AttractTutorial = nullptr;
		UEditorTutorial* LaunchTutorial = nullptr;
		FString BrowserFilter;
		GetDefault<UEditorTutorialSettings>()->FindTutorialInfoForContext(TEXT("LevelEditor"), AttractTutorial, LaunchTutorial, BrowserFilter);

		// prepare and send analytics data
		bool const bClosedInitialAttract = (CurrentTutorial == AttractTutorial);

		FString const CurrentExcerptTitle = bClosedInitialAttract ? TEXT("InitialAttract") : CurrentTutorial->Stages[CurrentTutorialStage].Name.ToString();
		int32 const CurrentExcerptIndex = bClosedInitialAttract ? -1 : CurrentTutorialStage;
		float const CurrentPageElapsedTime = bClosedInitialAttract ? 0.f : (float)(FPlatformTime::Seconds() - CurrentTutorialStartTime);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LastStageIndex"), CurrentExcerptIndex));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("LastStageTitle"), CurrentExcerptTitle));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TimeSpentInTutorial"), CurrentPageElapsedTime));
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TutorialAsset"), FIntroTutorials::AnalyticsEventNameFromTutorial(CurrentTutorial)));
			
		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.Closed"), EventAttributes );
	}

	//Tutorial is no longer current
	CurrentTutorial = nullptr;
}

bool STutorialRoot::WasWidgetDrawn(const FName& InName) const
{
	for(const auto& WidgetName : PreviouslyDrawnWidgets)
	{
		if(InName == WidgetName)
		{
			return true;
		}
	}

	return false;
}

void STutorialRoot::WidgetWasDrawn(const FName& InName)
{
	DrawnWidgets.Add(InName);
}

#undef LOCTEXT_NAMESPACE
