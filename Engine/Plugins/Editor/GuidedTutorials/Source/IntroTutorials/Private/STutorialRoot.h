// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "IIntroTutorials.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "Widgets/SWindow.h"

class SEditorTutorials;
class UEditorTutorial;

/**
 * The widget which simply monitors windows in its tick function to see if we need to attach
 * a tutorial overlay.
 */
class STutorialRoot : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS( STutorialRoot ) 
	{
		_Visibility = EVisibility::HitTestInvisible;
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** SWidget implementation */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** FGCObject implementation */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("STutorialRoot");
	}

	/** Launch the specified tutorial from the specified window */
	void LaunchTutorial(UEditorTutorial* InTutorial, IIntroTutorials::ETutorialStartType InStartType, TWeakPtr<SWindow> InNavigationWindow, FSimpleDelegate InOnTutorialClosed, FSimpleDelegate InOnTutorialExited);

	/** Close all tutorial content */
	void CloseAllTutorialContent();

	/** Reload tutorials that we know about */
	void ReloadTutorials();

	/** Got to the previous stage in the current tutorial */
	void GoToPreviousStage();

	/** Got to the next stage in the current tutorial */
	void GoToNextStage(TWeakPtr<SWindow> InNavigationWindow);

	/** Has this named widget been raw anywhere this frame? */
	bool WasWidgetDrawn(const FName& InName) const;

	/** Register that this widget was drawn this frame */
	void WidgetWasDrawn(const FName& InName);

	/** This is currently used for the "loading" widget */
	void AttachWidget(TSharedPtr<SWidget> Widget);

	/** This is currently used for the "loading" widget */
	void DetachWidget();

private:
	/** Handle when the next button is clicked */
	void HandleNextClicked(TWeakPtr<SWindow> InNavigationWindow);

	/** Handle when the back button is clicked */
	void HandleBackClicked();

	/** Handle when the home button is clickeds */
	void HandleHomeClicked();

	/** Handle when the close button is clicked */
	void HandleCloseClicked();

	/** Handle retrieving the current tutorial */
	UEditorTutorial* HandleGetCurrentTutorial();

	/** Handle retrieving the current tutorial stage */
	int32 HandleGetCurrentTutorialStage();

	/** Function called on Tick() to check active windows for whether they need an overlay adding */
	void MaybeAddOverlay(TSharedRef<SWindow> InWindow);

private:
	/** Container widgets, inserted into window overlays */
	TMap<TWeakPtr<SWindow>, TWeakPtr<SEditorTutorials>> TutorialWidgets;

	/** Tutorial we are currently viewing */
	TObjectPtr<UEditorTutorial> CurrentTutorial;

	/** Current stage of tutorial */
	int32 CurrentTutorialStage;

	/** Start time of the current tutorial, if any */
	float CurrentTutorialStartTime;

	/** Current drawn widgets for the last frame */
	TArray<FName> DrawnWidgets;
	TArray<FName> PreviouslyDrawnWidgets;
};
