// Copyright Epic Games, Inc. All Rights Reserved.

#include "STutorialsBrowser.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "Engine/Blueprint.h"
#include "Engine/Texture2D.h"
#include "AssetRegistry/AssetData.h"
#include "IntroTutorials.h"
#include "EditorTutorial.h"
#include "STutorialContent.h"
#include "TutorialSettings.h"
#include "EditorTutorialSettings.h"
#include "EditorTutorialStyle.h"
#include "TutorialStateSettings.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "UObject/GCObject.h"
#include "Slate/DeferredCleanupSlateBrush.h"

#define LOCTEXT_NAMESPACE "TutorialsBrowser"

class FTutorialListEntry_Tutorial;

DECLARE_DELEGATE_OneParam(FOnCategorySelected, const FString& /* InCategory */);

DEFINE_LOG_CATEGORY_STATIC(LogTutorials, Log, All);

namespace TutorialBrowserConstants
{
	const float RefreshTimerInterval = 1.0f;

	const float ProgressUpdateInterval = 0.5f;
}

class FTutorialListEntry_Category : public ITutorialListEntry, public TSharedFromThis<FTutorialListEntry_Category>
{
public:
	FTutorialListEntry_Category(FOnCategorySelected InOnCategorySelected)
		: OnCategorySelected(InOnCategorySelected)
		, SlateBrush(nullptr)
	{}

	FTutorialListEntry_Category(const FTutorialCategory& InCategory, FOnCategorySelected InOnCategorySelected, const TAttribute<FText>& InHighlightText)
		: Category(InCategory)
		, OnCategorySelected(InOnCategorySelected)
		, HighlightText(InHighlightText)
		, SlateBrush(nullptr)
	{
		if(!Category.Identifier.IsEmpty())
		{
			int32 Index = INDEX_NONE;
			if(Category.Identifier.FindLastChar(TEXT('.'), Index))
			{
				CategoryName = Category.Identifier.RightChop(Index + 1);
			}
			else
			{
				CategoryName = Category.Identifier;
			}
		}

		if(Category.Texture.IsValid())
		{
			UTexture2D* Texture = LoadObject<UTexture2D>(nullptr, *Category.Texture.ToString());
			if(Texture != nullptr)
			{
				FIntPoint TextureSize = Texture->GetImportedSize();
				DynamicBrush = FDeferredCleanupSlateBrush::CreateBrush(Texture, FVector2D((float)TextureSize.X, (float)TextureSize.Y));
				SlateBrush = DynamicBrush->GetSlateBrush();
			}
		}	

		if(SlateBrush == nullptr)
		{
			if(Category.Icon.Len() > 0)
			{
				SlateBrush = FAppStyle::Get().GetBrush(FName(*Category.Icon));
			}
		}

		if(SlateBrush == nullptr)
		{
			SlateBrush = FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.DefaultTutorialIcon");
		}
	}

	virtual ~FTutorialListEntry_Category()
	{
	}

	virtual TSharedRef<ITableRow> OnGenerateTutorialRow(const TSharedRef<STableViewBase>& OwnerTable) const override
	{
		return SNew(STableRow<TSharedPtr<ITutorialListEntry>>, OwnerTable)
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SNew(SButton)
				.OnClicked(this, &FTutorialListEntry_Category::OnClicked)
				.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Browser.Button"))
				.ForegroundColor(FSlateColor::UseForeground())
				.Content()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(8.0f)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SBox)
							.WidthOverride(64.0f)
							.HeightOverride(64.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							[
								SNew(SImage)
								.Image(SlateBrush)
							]
						]
						+ SOverlay::Slot()
						.VAlign(VAlign_Bottom)
						.HAlign(HAlign_Right)
						[
							SNew(SImage)
							.ToolTipText(LOCTEXT("CompletedCategoryCheckToolTip", "This category has been completed"))
							.Visibility(this, &FTutorialListEntry_Category::GetCompletedVisibility)
							.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.Completed"))
						]
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Text(GetTitleText())
							.TextStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>("Tutorials.Browser.SummaryHeader"))
							.HighlightText(HighlightText)
							.HighlightColor(FEditorTutorialStyle::Get().GetColor("Tutorials.Browser.HighlightTextColor"))
							.HighlightShape(FAppStyle::Get().GetBrush("TextBlock.HighlightShape"))
						]
						+SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(Category.Description)
							.TextStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>("Tutorials.Browser.SummaryText"))
							.HighlightText(HighlightText)
							.HighlightColor(FEditorTutorialStyle::Get().GetColor("Tutorials.Browser.HighlightTextColor"))
							.HighlightShape(FAppStyle::Get().GetBrush("TextBlock.HighlightShape"))
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Visibility(this, &FTutorialListEntry_Category::OnGetArrowVisibility)
						.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.CategoryArrow"))
					]
				]
			]
		];
	}

	bool PassesFilter(const FString& InCategoryFilter, const FString& InFilter) const override
	{
		const FString Title = !Category.Title.IsEmpty() ? Category.Title.ToString() : CategoryName;
		const bool bPassesFilter = InFilter.IsEmpty() || (Title.Contains(InFilter) || Category.Description.ToString().Contains(InFilter));
		const bool bPassesCategory = InCategoryFilter.IsEmpty() || Category.Identifier.StartsWith(InCategoryFilter);
		return bPassesFilter && bPassesCategory;
	}

	int32 GetSortOrder() const override
	{
		return Category.SortOrder;
	}

	FText GetTitleText() const override
	{
		return !Category.Title.IsEmpty() ? Category.Title : FText::FromString(CategoryName);
	}

	bool SortAgainst(TSharedRef<ITutorialListEntry> OtherEntry) const override
	{
		return (GetSortOrder() == OtherEntry->GetSortOrder()) ? (GetTitleText().CompareTo(OtherEntry->GetTitleText()) < 0) : (GetSortOrder() < OtherEntry->GetSortOrder());
	}

	void AddSubCategory(TSharedPtr<FTutorialListEntry_Category> InSubCategory)
	{
		SubCategories.Add(InSubCategory);
	}

	void AddTutorial(TSharedPtr<FTutorialListEntry_Tutorial> InTutorial);

	FReply OnClicked() const
	{
		if(SubCategories.Num() > 0 || Tutorials.Num() > 0)
		{
			OnCategorySelected.ExecuteIfBound(Category.Identifier);
		}
		return FReply::Handled();
	}

	EVisibility OnGetArrowVisibility() const
	{
		return (SubCategories.Num() > 0 || Tutorials.Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetCompletedVisibility() const override
	{
		for (int32 i = 0; i < Tutorials.Num(); ++i)
		{
			if (Tutorials[i].IsValid() && (Tutorials[i]->GetCompletedVisibility() != EVisibility::Visible))
			{
				return EVisibility::Hidden;
			}
		}
		for (int32 i = 0; i < SubCategories.Num(); ++i)
		{
			if (SubCategories[i].IsValid() && (SubCategories[i]->GetCompletedVisibility() != EVisibility::Visible))
			{
				return EVisibility::Hidden;
			}
		}
		return EVisibility::Visible;
	}

public:
	/** Copy of the category info */
	FTutorialCategory Category;

	/** Parent category */
	TWeakPtr<ITutorialListEntry> ParentCategory;

	/** Sub-categories */
	TArray<TSharedPtr<ITutorialListEntry>> SubCategories;

	/** Tutorials in this category */
	TArray<TSharedPtr<ITutorialListEntry>> Tutorials;

	/** Selection delegate */
	FOnCategorySelected OnCategorySelected;

	/** Name of the category, empty if this category is at the root */
	FString CategoryName;

	/** Text to highlight */
	TAttribute<FText> HighlightText;

	/** Static brush from the editor style */
	const FSlateBrush* SlateBrush;

	/** Dynamic brush from the texture specified by the user */
	TSharedPtr<FDeferredCleanupSlateBrush> DynamicBrush;
};

DECLARE_DELEGATE_TwoParams(FOnTutorialSelected, UEditorTutorial* /* InTutorial */, bool /* bRestart */ );

class FTutorialListEntry_Tutorial : public ITutorialListEntry, public TSharedFromThis<FTutorialListEntry_Tutorial>, public FGCObject
{
public:
	FTutorialListEntry_Tutorial(UEditorTutorial* InTutorial, FOnTutorialSelected InOnTutorialSelected, const TAttribute<FText>& InHighlightText)
		: Tutorial(InTutorial)
		, OnTutorialSelected(InOnTutorialSelected)
		, HighlightText(InHighlightText)
		, SlateBrush(nullptr)
		, LastUpdateTime(0.0f)
	{
		if(Tutorial->Texture != nullptr)
		{
			FIntPoint TextureSize = Tutorial->Texture->GetImportedSize();
			DynamicBrush = FDeferredCleanupSlateBrush::CreateBrush(Tutorial->Texture, FVector2D((float)TextureSize.X, (float)TextureSize.Y));
			SlateBrush = DynamicBrush->GetSlateBrush();
		}	
		else if(Tutorial->Icon.Len() > 0)
		{
			SlateBrush = FAppStyle::Get().GetBrush(FName(*Tutorial->Icon));
		}
		
		if(SlateBrush == nullptr)
		{
			SlateBrush = FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.DefaultTutorialIcon");
		}
	}

	virtual ~FTutorialListEntry_Tutorial()
	{
	}

	virtual TSharedRef<ITableRow> OnGenerateTutorialRow(const TSharedRef<STableViewBase>& OwnerTable) const override
	{
		CacheProgress();

		return SNew(STableRow<TSharedPtr<ITutorialListEntry>>, OwnerTable)
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 2.0f))
			[
				SAssignNew(LaunchButton, SButton)
				.OnClicked(this, &FTutorialListEntry_Tutorial::OnClicked, false)
				.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Browser.Button"))
				.ForegroundColor(FSlateColor::UseForeground())
				.Content()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Padding(8.0f)
						[
							SNew(SOverlay)
							+SOverlay::Slot()
							[
								SNew(SBox)
								.WidthOverride(64.0f)
								.HeightOverride(64.0f)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								[
									SNew(SImage)
									.Image(SlateBrush)
								]
							]
							+SOverlay::Slot()
							.VAlign(VAlign_Bottom)
							.HAlign(HAlign_Right)
							[
								SNew(SImage)
								.ToolTipText(LOCTEXT("CompletedTutorialCheckToolTip", "This tutorial has been completed"))
								.Visibility(this, &FTutorialListEntry_Tutorial::GetCompletedVisibility)
								.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.Completed"))
							]
						]
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.FillWidth(1.0f)
								[
									SNew(STextBlock)
									.Text(GetTitleText())
									.TextStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>("Tutorials.Browser.SummaryHeader"))
									.HighlightText(HighlightText)
									.HighlightColor(FEditorTutorialStyle::Get().GetColor("Tutorials.Browser.HighlightTextColor"))
									.HighlightShape(FAppStyle::Get().GetBrush("TextBlock.HighlightShape"))
								]
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SButton)
									.ToolTipText(LOCTEXT("RestartButtonToolTip", "Start this tutorial from the beginning"))
									.Visibility(this, &FTutorialListEntry_Tutorial::GetRestartVisibility)
									.OnClicked(this, &FTutorialListEntry_Tutorial::OnClicked, true)
									.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Browser.Button"))
									.Content()
									[
										SNew(SImage)
										.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.RestartButton"))
									]
								]
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SBox)
								.Visibility(this, &FTutorialListEntry_Tutorial::GetProgressVisibility)
								.HeightOverride(3.0f)
								[
									SNew(SProgressBar)
									.Percent(this, &FTutorialListEntry_Tutorial::GetProgress)
								]
							]
							+SVerticalBox::Slot()
							.FillHeight(1.0f)
							[
								STutorialContent::GenerateContentWidget(Tutorial->SummaryContent, DocumentationPage, HighlightText)
							]
						]
					]
				]
			]
		];
	}

	bool PassesFilter(const FString& InCategoryFilter, const FString& InFilter) const override
	{
		const bool bPassesFilter = InFilter.IsEmpty() || (Tutorial->Title.ToString().Contains(InFilter) || Tutorial->SummaryContent.Text.ToString().Contains(InFilter));
		const bool bPassesCategory = InCategoryFilter.IsEmpty() || Tutorial->Category.StartsWith(InCategoryFilter);

		return bPassesFilter && bPassesCategory;
	}

	FText GetTitleText() const override
	{
		return Tutorial->Title;
	}

	int32 GetSortOrder() const override
	{
		return Tutorial->SortOrder;
	}

	bool SortAgainst(TSharedRef<ITutorialListEntry> OtherEntry) const override
	{
		return (GetSortOrder() == OtherEntry->GetSortOrder()) ? (GetTitleText().CompareTo(OtherEntry->GetTitleText()) < 0) : (GetSortOrder() < OtherEntry->GetSortOrder());
	}

	FReply OnClicked(bool bRestart) const
	{
		OnTutorialSelected.ExecuteIfBound(Tutorial, bRestart);

		return FReply::Handled();
	}

	TOptional<float> GetProgress() const
	{
		CacheProgress();
		return Progress;
	}

	EVisibility GetProgressVisibility() const
	{
		if(LaunchButton->IsHovered())
		{
			CacheProgress();
			return LaunchButton->IsHovered() && bHaveSeenTutorial ? EVisibility::Visible : EVisibility::Hidden;
		}

		return EVisibility::Hidden;
	}

	EVisibility GetRestartVisibility() const
	{
		if(LaunchButton->IsHovered())
		{
			CacheProgress();
			return LaunchButton->IsHovered() && bHaveSeenTutorial ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}

	EVisibility GetCompletedVisibility() const override
	{
		CacheProgress();
		return bHaveCompletedTutorial ? EVisibility::Visible : EVisibility::Hidden;
	}

	void CacheProgress() const
	{
		if(FPlatformTime::Seconds() - LastUpdateTime > TutorialBrowserConstants::ProgressUpdateInterval)
		{
			bHaveCompletedTutorial = GetDefault<UTutorialStateSettings>()->HaveCompletedTutorial(Tutorial);
			bHaveSeenTutorial = false;
			const int32 CurrentStage = GetDefault<UTutorialStateSettings>()->GetProgress(Tutorial, bHaveSeenTutorial);
			Progress = (Tutorial->Stages.Num() > 0) ? (float)(CurrentStage + 1) / (float)Tutorial->Stages.Num() : 0.0f;

			LastUpdateTime = FPlatformTime::Seconds();
		}
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(Tutorial);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTutorialListEntry_Tutorial");
	}

public:
	/** Parent category */
	TWeakPtr<ITutorialListEntry> ParentCategory;

	/** Tutorial that we will launch */
	TObjectPtr<UEditorTutorial> Tutorial;

	/** Selection delegate */
	FOnTutorialSelected OnTutorialSelected;

	/** Text to highlight */
	TAttribute<FText> HighlightText;

	/** Button clicked to launch tutorial */
	mutable TSharedPtr<SWidget> LaunchButton;

	/** Documentation page reference to use if we are displaying a UDN doc */
	mutable TSharedPtr<IDocumentationPage> DocumentationPage;

	/** Static brush from the editor style */
	const FSlateBrush* SlateBrush;

	/** Dynamic brush from the texture specified by the user */
	TSharedPtr<FDeferredCleanupSlateBrush> DynamicBrush;

	/** Cached tutorial completion state */
	mutable bool bHaveCompletedTutorial;

	/** Cached tutorial seen state */
	mutable bool bHaveSeenTutorial;

	/** Cached tutorial progress */
	mutable float Progress;

	/** Last update time */
	mutable float LastUpdateTime;
};

void STutorialsBrowser::Construct(const FArguments& InArgs)
{
	bNeedsRefresh = false;
	RefreshTimer = TutorialBrowserConstants::RefreshTimerInterval;

	OnClosed = InArgs._OnClosed;
	OnLaunchTutorial = InArgs._OnLaunchTutorial;
	ParentWindow = InArgs._ParentWindow;
	ExternalCategories = InArgs._ExternalCategories;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &STutorialsBrowser::HandleAssetAdded);

	RegisterActiveTimer( TutorialBrowserConstants::RefreshTimerInterval, FWidgetActiveTimerDelegate::CreateSP( this, &STutorialsBrowser::TriggerReloadTutorials ) );

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.OnClicked(this, &STutorialsBrowser::OnBackButtonClicked)
					.IsEnabled(this, &STutorialsBrowser::IsBackButtonEnabled)
					.ButtonStyle(&FEditorTutorialStyle::Get().GetWidgetStyle<FButtonStyle>("Tutorials.Browser.BackButton"))
					.ForegroundColor(FSlateColor::UseForeground())
					.Content()
					[
						SNew(SImage)
						.Image(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.BackButton.Image"))
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 1.0f)
					[
						SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<TSharedPtr<ITutorialListEntry>>)
						.ButtonContentPadding(FMargin(1.0f, 1.0f))
						.DelimiterImage(FEditorTutorialStyle::Get().GetBrush("Tutorials.Browser.Breadcrumb"))
						.TextStyle(FEditorTutorialStyle::Get(), "Tutorials.Browser.PathText")
						.ShowLeadingDelimiter( true )
						.OnCrumbClicked(this, &STutorialsBrowser::OnBreadcrumbClicked)
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 1.0f)
					[
						SNew(SSearchBox)
						.OnTextChanged(this, &STutorialsBrowser::OnSearchTextChanged)
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(TutorialList, SListView<TSharedPtr<ITutorialListEntry>>)
				.ListItemsSource(&FilteredEntries)
				.OnGenerateRow(this, &STutorialsBrowser::OnGenerateTutorialRow)
				.SelectionMode(ESelectionMode::None)
			]
		]
	];

	ReloadTutorials();

	RebuildCrumbs();
}

inline void FTutorialListEntry_Category::AddTutorial(TSharedPtr<FTutorialListEntry_Tutorial> InTutorial)
{
	Tutorials.Add(InTutorial);
}

EActiveTimerReturnType STutorialsBrowser::TriggerReloadTutorials( double InCurrentTime, float InDeltaTime )
{
	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		ReloadTutorials();
	}
	
	return EActiveTimerReturnType::Continue;
}

void STutorialsBrowser::SetFilter(const FString& InFilter)
{
	CategoryFilter = InFilter;
	ReloadTutorials();
}

TSharedRef<ITableRow> STutorialsBrowser::OnGenerateTutorialRow(TSharedPtr<ITutorialListEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return InItem->OnGenerateTutorialRow(OwnerTable);
}

TSharedPtr<FTutorialListEntry_Category> STutorialsBrowser::RebuildCategories()
{
	TArray<TSharedPtr<FTutorialListEntry_Category>> Categories;

	// add root category
	TSharedPtr<FTutorialListEntry_Category> RootCategory = MakeShareable(new FTutorialListEntry_Category(FOnCategorySelected::CreateSP(this, &STutorialsBrowser::OnCategorySelected)));
	Categories.Add(RootCategory);

	// add external categories
	for (const auto& TutorialCategory : ExternalCategories)
	{
		Categories.Add(MakeShareable(new FTutorialListEntry_Category(TutorialCategory, FOnCategorySelected::CreateSP(this, &STutorialsBrowser::OnCategorySelected), TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &STutorialsBrowser::GetSearchText)))));
	}

	// rebuild categories
	for(const auto& TutorialCategory : GetDefault<UTutorialSettings>()->Categories)
	{
		Categories.Add(MakeShareable(new FTutorialListEntry_Category(TutorialCategory, FOnCategorySelected::CreateSP(this, &STutorialsBrowser::OnCategorySelected), TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &STutorialsBrowser::GetSearchText)))));
	}

	for(const auto& TutorialCategory : GetDefault<UEditorTutorialSettings>()->Categories)
	{
		Categories.Add(MakeShareable(new FTutorialListEntry_Category(TutorialCategory, FOnCategorySelected::CreateSP(this, &STutorialsBrowser::OnCategorySelected), TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &STutorialsBrowser::GetSearchText)))));
	}

	for(auto& Category : Categories)
	{
		// Figure out which base category this category belongs in
		TSharedPtr<FTutorialListEntry_Category> ParentCategory = RootCategory;
		const FString& CategoryPath = Category->Category.Identifier;

		// We're expecting the category string to be in the "A.B.C" format.  We'll split up the string here and form
		// a proper hierarchy in the UI
		TArray< FString > SplitCategories;
		CategoryPath.ParseIntoArray( SplitCategories, TEXT( "." ), true /* bCullEmpty */ );

		FString CurrentCategoryPath;

		// Make sure all of the categories exist
		for(const auto& SplitCategory : SplitCategories)
		{
			// Locate this category at the level we're at in the hierarchy
			TSharedPtr<FTutorialListEntry_Category> FoundCategory = NULL;
			TArray< TSharedPtr<ITutorialListEntry> >& TestCategoryList = ParentCategory.IsValid() ? ParentCategory->SubCategories : RootCategory->SubCategories;
			for(auto& TestCategory : TestCategoryList)
			{
				if( StaticCastSharedPtr<FTutorialListEntry_Category>(TestCategory)->CategoryName == SplitCategory )
				{
					// Found it!
					FoundCategory = StaticCastSharedPtr<FTutorialListEntry_Category>(TestCategory);
					break;
				}
			}

			if(!CurrentCategoryPath.IsEmpty())
			{
				CurrentCategoryPath += TEXT(".");
			}

			CurrentCategoryPath += SplitCategory;

			if( !FoundCategory.IsValid() )
			{
				// OK, this is a new category name for us, so add it now!
				if(CategoryPath == CurrentCategoryPath)
				{
					FoundCategory = Category;
				}
				else
				{
					FTutorialCategory InterveningCategory;
					InterveningCategory.Identifier = CurrentCategoryPath;
					FoundCategory = MakeShareable(new FTutorialListEntry_Category(InterveningCategory, FOnCategorySelected::CreateSP(this, &STutorialsBrowser::OnCategorySelected), TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &STutorialsBrowser::GetSearchText))));
				}

				FoundCategory->ParentCategory = ParentCategory;
				TestCategoryList.Add( FoundCategory );
			}

			// Descend the hierarchy for the next category
			ParentCategory = FoundCategory;
		}
	}

	return RootCategory;
}

void STutorialsBrowser::RebuildTutorials(TSharedPtr<FTutorialListEntry_Category> InRootCategory)
{
	TArray<TSharedPtr<FTutorialListEntry_Tutorial>> Tutorials;

	//Ensure that tutorials are loaded into the asset registry before making a list of them.
	FAssetRegistryModule& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// rebuild tutorials
	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	Filter.TagsAndValues.Add(FBlueprintTags::NativeParentClassPath, FObjectPropertyBase::GetExportPath(UEditorTutorial::StaticClass()));
	Filter.TagsAndValues.Add(FBlueprintTags::ParentClassPath, FObjectPropertyBase::GetExportPath(UEditorTutorial::StaticClass()));

	TArray<FAssetData> AssetData;
	AssetRegistry.Get().GetAssets(Filter, AssetData);

	TArray<FString> DeletedTabs;
	DeletedTabs.Add("LevelEditorToolBox");

	for (const auto& TutorialAsset : AssetData)
	{
		UBlueprint* Blueprint = LoadObject<UBlueprint>(nullptr, *TutorialAsset.GetObjectPathString());
		if (Blueprint && Blueprint->GeneratedClass && Blueprint->BlueprintType == BPTYPE_Normal)
		{
			UEditorTutorial* Tutorial = NewObject<UEditorTutorial>(GetTransientPackage(), Blueprint->GeneratedClass);
			//UE-45734 - Loading the default object causes landscape tutorials to crash.
			//UEditorTutorial* Tutorial = Blueprint->GeneratedClass->GetDefaultObject<UEditorTutorial>();
			if(!Tutorial->bHideInBrowser)
			{
				bool bUsesDeletedTab = false;
				for (const FTutorialStage& Stage : Tutorial->Stages)
				{
					for (const FTutorialWidgetContent& TutorialWidgetContent : Stage.WidgetContent)
					{
						if (DeletedTabs.Contains(TutorialWidgetContent.WidgetAnchor.TabToFocusOrOpen))
						{
							bUsesDeletedTab = true;
							UE_LOGF(LogTutorials, Warning, "Tutorial uses deleted tab: %ls, tutorial: %ls", *TutorialWidgetContent.WidgetAnchor.TabToFocusOrOpen, *Blueprint->GetFullName());
							break;
						}
					}

					if (bUsesDeletedTab)
					{
						break;
					}
				}

				if (!bUsesDeletedTab)
				{
					Tutorials.Add(MakeShareable(new FTutorialListEntry_Tutorial(Tutorial, FOnTutorialSelected::CreateSP(this, &STutorialsBrowser::OnTutorialSelected), TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &STutorialsBrowser::GetSearchText)))));
				}
			}
		}
	}

	// add tutorials to categories
	for(const auto& Tutorial : Tutorials)
	{
		// Figure out which base category this tutorial belongs in
		TSharedPtr<FTutorialListEntry_Category> CategoryForTutorial = InRootCategory;
		const FString& CategoryPath = Tutorial->Tutorial->Category;

		// We're expecting the category string to be in the "A.B.C" format.  We'll split up the string here and form
		// a proper hierarchy in the UI
		TArray< FString > SplitCategories;
		CategoryPath.ParseIntoArray( SplitCategories, TEXT( "." ), true /* bCullEmpty */ );

		FString CurrentCategoryPath;

		// Make sure all of the categories exist
		for(const auto& SplitCategory : SplitCategories)
		{
			// Locate this category at the level we're at in the hierarchy
			TSharedPtr<FTutorialListEntry_Category> FoundCategory = NULL;
			TArray< TSharedPtr<ITutorialListEntry> >& TestCategoryList = CategoryForTutorial.IsValid() ? CategoryForTutorial->SubCategories : InRootCategory->SubCategories;
			for(auto& TestCategory : TestCategoryList)
			{
				if( StaticCastSharedPtr<FTutorialListEntry_Category>(TestCategory)->CategoryName == SplitCategory )
				{
					// Found it!
					FoundCategory = StaticCastSharedPtr<FTutorialListEntry_Category>(TestCategory);
					break;
				}
			}

			if(!CurrentCategoryPath.IsEmpty())
			{
				CurrentCategoryPath += TEXT(".");
			}

			CurrentCategoryPath += SplitCategory;

			if( !FoundCategory.IsValid() )
			{
				// OK, this is a new category name for us, so add it now!
				FTutorialCategory InterveningCategory;
				InterveningCategory.Identifier = CurrentCategoryPath;

				FoundCategory = MakeShareable(new FTutorialListEntry_Category(InterveningCategory, FOnCategorySelected::CreateSP(this, &STutorialsBrowser::OnCategorySelected), TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &STutorialsBrowser::GetSearchText))));
				FoundCategory->ParentCategory = CategoryForTutorial;
				TestCategoryList.Add( FoundCategory );
			}

			// Descend the hierarchy for the next category
			CategoryForTutorial = FoundCategory;
		}

		Tutorial->ParentCategory = CategoryForTutorial;
		CategoryForTutorial->AddTutorial( Tutorial );
	}
}

void STutorialsBrowser::ReloadTutorials()
{
	TSharedPtr<FTutorialListEntry_Category> RootCategory = RebuildCategories();
	RebuildTutorials(RootCategory);
	RootEntry = RootCategory;

	// now filter & arrange available tutorials
	FilterTutorials();
}

FReply STutorialsBrowser::OnCloseButtonClicked()
{
	OnClosed.ExecuteIfBound();
	return FReply::Handled();
}

FReply STutorialsBrowser::OnBackButtonClicked()
{
	TSharedPtr<FTutorialListEntry_Category> CurrentCategory = FindCategory_Recursive(RootEntry);
	if(CurrentCategory.IsValid() && CurrentCategory->ParentCategory.IsValid())
	{
		TSharedPtr<FTutorialListEntry_Category> PinnedParentCategory = StaticCastSharedPtr<FTutorialListEntry_Category>(CurrentCategory->ParentCategory.Pin());
		if(PinnedParentCategory.IsValid())
		{
			NavigationFilter = PinnedParentCategory->Category.Identifier;
			FilterTutorials();
		}
	}

	RebuildCrumbs();

	return FReply::Handled();
}

bool STutorialsBrowser::IsBackButtonEnabled() const
{
	if(CurrentCategoryPtr.IsValid())
	{
		return CurrentCategoryPtr.Pin()->ParentCategory.IsValid();
	}

	return false;
}

void STutorialsBrowser::OnTutorialSelected(UEditorTutorial* InTutorial, bool bRestart)
{
	if (InTutorial != nullptr)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Restarted"), bRestart));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TutorialAsset"), FIntroTutorials::AnalyticsEventNameFromTutorial(InTutorial)));

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Rocket.Tutorials.LaunchedFromBrowser"), EventAttributes);
		}
		//Close the tutorial browser so it doesn't get in the way of the actual tutorial.
		if (OnLaunchTutorial.IsBound())
		{
			FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
			IntroTutorials.DismissTutorialBrowser();
		}
	}
	OnLaunchTutorial.ExecuteIfBound(InTutorial, bRestart ? IIntroTutorials::ETutorialStartType::TST_RESTART : IIntroTutorials::ETutorialStartType::TST_CONTINUE, ParentWindow, FSimpleDelegate(), FSimpleDelegate());
}

void STutorialsBrowser::OnCategorySelected(const FString& InCategory)
{
	NavigationFilter = InCategory;
	FilterTutorials();

	RebuildCrumbs();
}

void STutorialsBrowser::FilterTutorials()
{
	FilteredEntries.Empty();

	if(SearchFilter.IsEmpty())
	{
		TSharedPtr<FTutorialListEntry_Category> CurrentCategory = FindCategory_Recursive(RootEntry);

		if(CurrentCategory.IsValid())
		{
			for(const auto& SubCategory : CurrentCategory->SubCategories)
			{
				if(SubCategory->PassesFilter(CategoryFilter, SearchFilter.ToString()))
				{
					FilteredEntries.Add(SubCategory);
				}
			}

			for(const auto& Tutorial : CurrentCategory->Tutorials)
			{
				if(Tutorial->PassesFilter(CategoryFilter, SearchFilter.ToString()))
				{
					FilteredEntries.Add(Tutorial);
				}
			}

			CurrentCategoryPtr = CurrentCategory;
		}
	}
	else
	{
		struct Local
		{
			static void AddSubCategory_Recursive(const FString& InCategoryFilter, const FString& InSearchFilter, TSharedPtr<FTutorialListEntry_Category> InCategory, TArray<TSharedPtr<ITutorialListEntry>>& InOutFilteredEntries)
			{
				if(InCategory.IsValid())
				{
					for(const auto& SubCategory : InCategory->SubCategories)
					{
						if(SubCategory->PassesFilter(InCategoryFilter, InSearchFilter))
						{
							InOutFilteredEntries.Add(SubCategory);
						}

						AddSubCategory_Recursive(InCategoryFilter, InSearchFilter, StaticCastSharedPtr<FTutorialListEntry_Category>(SubCategory), InOutFilteredEntries);
					}

					for(const auto& Tutorial : InCategory->Tutorials)
					{
						if(Tutorial->PassesFilter(InCategoryFilter, InSearchFilter))
						{
							InOutFilteredEntries.Add(Tutorial);
						}
					}
				}
			};
		};

		TSharedPtr<FTutorialListEntry_Category> CurrentCategory = FindCategory_Recursive(RootEntry);
		if(CurrentCategory.IsValid())
		{
			Local::AddSubCategory_Recursive(CategoryFilter, SearchFilter.ToString(), CurrentCategory, FilteredEntries);
			CurrentCategoryPtr = CurrentCategory;
		}
	}

	FilteredEntries.Sort(
		[](TSharedPtr<ITutorialListEntry> EntryA, TSharedPtr<ITutorialListEntry> EntryB)->bool
		{
			if(EntryA.IsValid() && EntryB.IsValid())
			{
				return EntryA->SortAgainst(EntryB.ToSharedRef());
			}
			return false;
		}
	);

	TutorialList->RequestListRefresh();
}

TSharedPtr<FTutorialListEntry_Category> STutorialsBrowser::FindCategory_Recursive(TSharedPtr<FTutorialListEntry_Category> InCategory) const
{
	if(InCategory.IsValid())
	{
		if(InCategory->Category.Identifier == NavigationFilter)
		{
			return InCategory;
		}

		for(const auto& Category : InCategory->SubCategories)
		{
			TSharedPtr<FTutorialListEntry_Category> TestCategory = FindCategory_Recursive(StaticCastSharedPtr<FTutorialListEntry_Category>(Category));
			if(TestCategory.IsValid())
			{
				return TestCategory;
			}
		}
	}

	return TSharedPtr<FTutorialListEntry_Category>();
}

void STutorialsBrowser::OnSearchTextChanged(const FText& InText)
{
	SearchFilter = InText;
	FilterTutorials();
}

FText STutorialsBrowser::GetSearchText() const
{
	return SearchFilter;
}

void STutorialsBrowser::OnBreadcrumbClicked(const TSharedPtr<ITutorialListEntry>& InEntry)
{
	TSharedPtr<ITutorialListEntry> ClickedEntry = InEntry;

	if(ClickedEntry.IsValid())
	{
		NavigationFilter = StaticCastSharedPtr<FTutorialListEntry_Category>(ClickedEntry)->Category.Identifier;
	}
	else
	{
		NavigationFilter.Empty();
	}
	
	RebuildCrumbs();

	FilterTutorials();
}

void STutorialsBrowser::RebuildCrumbs()
{
	BreadcrumbTrail->ClearCrumbs();

	// rebuild crumbs to this point
	TArray<TSharedPtr<FTutorialListEntry_Category>> Entries;
	TSharedPtr<FTutorialListEntry_Category> CurrentCategory = FindCategory_Recursive(RootEntry);
	if(CurrentCategory.IsValid())
	{
		TSharedPtr<FTutorialListEntry_Category> Category = StaticCastSharedPtr<FTutorialListEntry_Category>(CurrentCategory);
		while(Category.IsValid())
		{
			Entries.Add(Category);
			if(Category->ParentCategory.IsValid())
			{
				Category = StaticCastSharedPtr<FTutorialListEntry_Category>(Category->ParentCategory.Pin());
			}
			else
			{
				break;
			}
		}
	}

	for(int32 Index = Entries.Num() - 1; Index >= 0; Index--)
	{
		TSharedPtr<FTutorialListEntry_Category> Entry = Entries[Index];
		if(RootEntry == Entry)
		{
			BreadcrumbTrail->PushCrumb(LOCTEXT("PathRoot", "Tutorials"), TSharedPtr<ITutorialListEntry>());
		}
		else
		{
			BreadcrumbTrail->PushCrumb(Entry->GetTitleText(), Entry);
		}
	}
}

void STutorialsBrowser::HandleAssetAdded(const FAssetData& InAssetData)
{
	if(InAssetData.AssetClassPath == UBlueprint::StaticClass()->GetClassPathName())
	{
		const FString ParentClassPath = InAssetData.GetTagValueRef<FString>("ParentClass");
		if(!ParentClassPath.IsEmpty())
		{
			UClass* ParentClass = FindObject<UClass>(nullptr, *ParentClassPath);
			if(ParentClass == UEditorTutorial::StaticClass())
			{
				bNeedsRefresh = true;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
