// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorTutorial.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "IIntroTutorials.h"

struct FAssetData;
class FTutorialListEntry_Category;
class SWindow;
class UEditorTutorial;

DECLARE_DELEGATE_FiveParams(FOnLaunchTutorial, UEditorTutorial* /** InTutorialToLaunch */, IIntroTutorials::ETutorialStartType /* InStartType */, TWeakPtr<SWindow> /* InFromWindow */, FSimpleDelegate /* InOnTutorialClosed */, FSimpleDelegate /* InOnTutorialExited */);

/** Abstract base class for list entries in the tutorial menu */
struct ITutorialListEntry
{
	/** Generate content for a tree entry */
	virtual TSharedRef<ITableRow> OnGenerateTutorialRow(const TSharedRef<STableViewBase>& OwnerTable) const = 0;

	/** Whether this entry passes the current filter criteria */
	virtual bool PassesFilter(const FString& InCategoryFilter, const FString& InFilter) const = 0;

	/** Get the text representation of this item's title */
	virtual FText GetTitleText() const = 0;

	/** Get a priority value to override alphabetical sorting */
	virtual int32 GetSortOrder() const = 0;

	/** 
	 * Sort this entry against another entry
	 * @return true if this < OtherEntry 
	 */
	virtual bool SortAgainst(TSharedRef<ITutorialListEntry> OtherEntry) const = 0;

	/** Return true if this entry should show up as completed (currently used to hide/show green check mark) */
	virtual EVisibility GetCompletedVisibility() const = 0;
};

class FTutorialListEntry_Category;

/**
 * The widget which holds all available tutorials
 */
class STutorialsBrowser : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( STutorialsBrowser ) {}

	SLATE_ARGUMENT(FSimpleDelegate, OnClosed)
	SLATE_ARGUMENT(FOnLaunchTutorial, OnLaunchTutorial)
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
	SLATE_ARGUMENT(TArray<FTutorialCategory>, ExternalCategories)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	/** Set the current filter string. Filters are used to only show the specified category of tutorials (e.g. only Blueprint tutorials) */
	void SetFilter(const FString& InFilter);

	/** Reload all tutorials that we know about */
	void ReloadTutorials();

protected:
	/** Handle generating a table row in the browser */
	TSharedRef<ITableRow> OnGenerateTutorialRow(TSharedPtr<ITutorialListEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Handle closing the browser */
	FReply OnCloseButtonClicked();

	/** Handle traversing back up the browser hierarchy */
	FReply OnBackButtonClicked();

	/** Handle whether the back button can be clicked */
	bool IsBackButtonEnabled() const;

	/** Delegate handler fired when a tutorial is selected form the browser */
	void OnTutorialSelected(UEditorTutorial* InTutorial, bool bRestart);

	/** Delegate handler fired when a category is selected form the browser */
	void OnCategorySelected(const FString& InCategory);

	/** Filter displayed tutorials - regenerates the displayed items */
	void FilterTutorials();

	/** Rebuild the displayed categories */
	TSharedPtr<FTutorialListEntry_Category> RebuildCategories();

	/** Rebuild the displayed tutorials */
	void RebuildTutorials(TSharedPtr<FTutorialListEntry_Category> InRootCategory);

	/** Recursive helper function used to find a category given the current NavigationFilter */
	TSharedPtr<FTutorialListEntry_Category> FindCategory_Recursive(TSharedPtr<FTutorialListEntry_Category> InCategory) const;

	/** Handle rebuilding the browser display when the filter text changes */
	void OnSearchTextChanged(const FText& InText);

	/** Supplies the text to display in the filter box */
	FText GetSearchText() const;

	/** Handle clicking the breadcrumb trail */
	void OnBreadcrumbClicked(const TSharedPtr<ITutorialListEntry>& InEntry);

	/** Rebuild the breadcrumb trail according to the current category */
	void RebuildCrumbs();

	/** Handle an asset being added - rebuild our list if required */
	void HandleAssetAdded(const FAssetData& InAssetData);

private:

	/** Triggers a reload of the tutorials */
	EActiveTimerReturnType TriggerReloadTutorials( double InCurrentTime, float InDeltaTime );

	/** Root entry of the tutorials tree */
	TSharedPtr<FTutorialListEntry_Category> RootEntry;

	/** Current filtered entries */
	TArray<TSharedPtr<ITutorialListEntry>> FilteredEntries;

	/** List of tutorials widget */
	TSharedPtr<SListView<TSharedPtr<ITutorialListEntry>>> TutorialList;

	/** Delegate fired when the close button is clicked */
	FSimpleDelegate OnClosed;

	/** Delegate fired when a tutorial is selected */
	FOnLaunchTutorial OnLaunchTutorial;

	/** Current search string */
	FText SearchFilter;

	/** Current category navigation string */
	FString NavigationFilter;

	/** Static category filter for this browser */
	FString CategoryFilter;

	/** Cached current category */
	TWeakPtr<FTutorialListEntry_Category> CurrentCategoryPtr;

	/** Parent window of this browser */
	TWeakPtr<SWindow> ParentWindow;

	/** Breadcrumb trail for path display */
	TSharedPtr<SBreadcrumbTrail<TSharedPtr<ITutorialListEntry>>> BreadcrumbTrail;

	/** Whether we need to refresh the content in the browser */
	bool bNeedsRefresh;

	/** Prevent us from refreshing too often */
	float RefreshTimer;

	/** Categories that were registered externally, i.e. by a plugin */
	TArray<FTutorialCategory> ExternalCategories;
};
