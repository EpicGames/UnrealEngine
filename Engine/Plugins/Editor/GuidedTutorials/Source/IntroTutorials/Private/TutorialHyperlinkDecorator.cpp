// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialHyperlinkDecorator.h"
#include "Styling/CoreStyle.h"
#include "TutorialHyperlinkRun.h"
#include "EditorTutorialStyle.h"

TSharedRef< FTutorialHyperlinkDecorator > FTutorialHyperlinkDecorator::Create( FString Id, const FSlateHyperlinkRun::FOnClick& NavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate )
{
	return MakeShareable( new FTutorialHyperlinkDecorator( Id, NavigateDelegate, InToolTipTextDelegate, InToolTipDelegate ) );
}

FTutorialHyperlinkDecorator::FTutorialHyperlinkDecorator( FString InId, const FSlateHyperlinkRun::FOnClick& InNavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate )
	: FHyperlinkDecorator(InId, InNavigateDelegate, InToolTipTextDelegate, InToolTipDelegate)
{
}

TSharedRef< ISlateRun > FTutorialHyperlinkDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style)
{
	FString StyleName = TEXT("Hyperlink");

	const FTextRange* const MetaDataStyleNameRange = RunParseResult.MetaData.Find( TEXT("style") );
	if ( MetaDataStyleNameRange != NULL )
	{
		const FString MetaDataStyleName = OriginalText.Mid(MetaDataStyleNameRange->BeginIndex, MetaDataStyleNameRange->EndIndex - MetaDataStyleNameRange->BeginIndex);
		StyleName = *MetaDataStyleName;
	}

	FTextRange ModelRange;
	ModelRange.BeginIndex = InOutModelText->Len();
	*InOutModelText += OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex);
	ModelRange.EndIndex = InOutModelText->Len();

	if ( !Style->HasWidgetStyle<FHyperlinkStyle>( FName( *StyleName ) ) )
	{
		Style = &FEditorTutorialStyle::Get();
	}

	FRunInfo RunInfo( RunParseResult.Name );
	for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid( Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	return FTutorialHyperlinkRun::Create( RunInfo, InOutModelText, Style->GetWidgetStyle<FHyperlinkStyle>( FName( *StyleName ) ), NavigateDelegate, ToolTipDelegate, ToolTipTextDelegate, ModelRange );
}
