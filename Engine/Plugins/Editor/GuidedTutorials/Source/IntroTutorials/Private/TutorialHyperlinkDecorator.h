// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/TextDecorators.h"

class ISlateStyle;

class FTutorialHyperlinkDecorator : public FHyperlinkDecorator
{
public:

	static TSharedRef< FTutorialHyperlinkDecorator > Create( FString InId, const FSlateHyperlinkRun::FOnClick& InNavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate );

public:

	virtual TSharedRef< ISlateRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override;

protected:

	FTutorialHyperlinkDecorator( FString InId, const FSlateHyperlinkRun::FOnClick& InNavigateDelegate, const FSlateHyperlinkRun::FOnGetTooltipText& InToolTipTextDelegate, const FSlateHyperlinkRun::FOnGenerateTooltip& InToolTipDelegate );
};
