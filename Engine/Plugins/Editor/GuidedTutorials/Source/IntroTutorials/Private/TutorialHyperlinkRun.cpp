// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialHyperlinkRun.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Text/WidgetLayoutBlock.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SRichTextHyperlink.h"
#include "EditorTutorialStyle.h"

TSharedRef< FTutorialHyperlinkRun > FTutorialHyperlinkRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate )
{
	return MakeShareable( new FTutorialHyperlinkRun( InRunInfo, InText, InStyle, NavigateDelegate, InTooltipDelegate, InTooltipTextDelegate ) );
}

TSharedRef< FTutorialHyperlinkRun > FTutorialHyperlinkRun::Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange )
{
	return MakeShareable( new FTutorialHyperlinkRun( InRunInfo, InText, InStyle, NavigateDelegate, InTooltipDelegate, InTooltipTextDelegate, InRange ) );
}

FVector2D FTutorialHyperlinkRun::Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const 
{
	FVector2D Measurement = FSlateHyperlinkRun::Measure(StartIndex, EndIndex, Scale, TextContext);

	if(bIsBrowserLink && EndIndex == Range.EndIndex)
	{
		Measurement.X += LinkBrush->ImageSize.X;
	}

	return Measurement;
}

TSharedRef< ILayoutBlock > FTutorialHyperlinkRun::CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer )
{
	FText ToolTipText;
	TSharedPtr<IToolTip> ToolTip;
	
	if(TooltipDelegate.IsBound())
	{
		ToolTip = TooltipDelegate.Execute(RunInfo.MetaData);
	}
	else
	{
		const FString* Url = RunInfo.MetaData.Find(TEXT("href"));
		if(TooltipTextDelegate.IsBound())
		{
			ToolTipText = TooltipTextDelegate.Execute(RunInfo.MetaData);
		}
		else if(Url != nullptr)
		{
			ToolTipText = FText::FromString(*Url);
		}
	}

	TSharedRef<SHorizontalBox> Widget = 
		SNew(SHorizontalBox)
		.ToolTipText(ToolTipText)
		.ToolTip(ToolTip)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SRichTextHyperlink, ViewModel )
			.Style( &Style )
			.Text( FText::FromString( FString::ConstructFromPtrSize( **Text + StartIndex, EndIndex - StartIndex ) ) )
			.ToolTip( ToolTip )
			.ToolTipText( ToolTipText )
			.OnNavigate( this, &FTutorialHyperlinkRun::OnNavigate )
			.TextShapingMethod( TextContext.TextShapingMethod )
		];

	if(bIsBrowserLink && EndIndex == Range.EndIndex)
	{
		Widget->AddSlot()
		.AutoWidth()
		[
			SNew(SImage)
			.Image(LinkBrush)
		];
	}
	
	// We need to do a prepass here as CreateBlock can be called after the main Slate prepass has been run, 
	// which can result in the hyperlink widget not being correctly setup before it is painted
	Widget->SlatePrepass();

	Children.Add( Widget );

	return FWidgetLayoutBlock::Create( SharedThis( this ), Widget, FTextRange( StartIndex, EndIndex ), Size, TextContext, Renderer );
}

FTutorialHyperlinkRun::FTutorialHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate ) 
	: FSlateHyperlinkRun(InRunInfo, InText, InStyle, InNavigateDelegate, InTooltipDelegate, InTooltipTextDelegate)
	, bIsBrowserLink(CheckIsBrowserLink())
	, LinkBrush(FEditorTutorialStyle::Get().GetBrush("Tutorials.Content.ExternalLink"))
{

}

FTutorialHyperlinkRun::FTutorialHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange ) 
	: FSlateHyperlinkRun(InRunInfo, InText, InStyle, InNavigateDelegate, InTooltipDelegate, InTooltipTextDelegate,InRange)
	, bIsBrowserLink(CheckIsBrowserLink())
	, LinkBrush(FEditorTutorialStyle::Get().GetBrush("Tutorials.Content.ExternalLink"))
{

}

FTutorialHyperlinkRun::FTutorialHyperlinkRun( const FTutorialHyperlinkRun& Run ) 
	: FSlateHyperlinkRun(Run)
	, bIsBrowserLink(Run.bIsBrowserLink)
	, LinkBrush(Run.LinkBrush)
{

}

bool FTutorialHyperlinkRun::CheckIsBrowserLink() const
{
	const FString* IdMetaData = RunInfo.MetaData.Find(TEXT("id"));
	return (IdMetaData != nullptr && (*IdMetaData == TEXT("browser") || *IdMetaData == TEXT("udn")));
}
