// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/ILayoutBlock.h"
#include "Framework/Text/SlateHyperlinkRun.h"

class FTutorialHyperlinkRun : public FSlateHyperlinkRun
{
public:

	static TSharedRef< FTutorialHyperlinkRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate );
																																	 
	static TSharedRef< FTutorialHyperlinkRun > Create( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick NavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange );

public:

	virtual ~FTutorialHyperlinkRun() {}

	/** FSlateHyperlinkRun implementation */
	virtual FVector2D Measure( int32 StartIndex, int32 EndIndex, float Scale, const FRunTextContext& TextContext ) const override;
	virtual TSharedRef< ILayoutBlock > CreateBlock( int32 StartIndex, int32 EndIndex, FVector2D Size, const FLayoutBlockTextContext& TextContext, const TSharedPtr< IRunRenderer >& Renderer ) override;

private:

	FTutorialHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate );
																										 
	FTutorialHyperlinkRun( const FRunInfo& InRunInfo, const TSharedRef< const FString >& InText, const FHyperlinkStyle& InStyle, FOnClick InNavigateDelegate, FOnGenerateTooltip InTooltipDelegate, FOnGetTooltipText InTooltipTextDelegate, const FTextRange& InRange );

	FTutorialHyperlinkRun( const FTutorialHyperlinkRun& Run );

private:
	// check whether our metadata makes us a browser link
	bool CheckIsBrowserLink() const;

private:
	/** Whether we are an external (browser) link or not */
	bool bIsBrowserLink;

	/** The brush we use to display external links */
	const FSlateBrush* LinkBrush;
};
