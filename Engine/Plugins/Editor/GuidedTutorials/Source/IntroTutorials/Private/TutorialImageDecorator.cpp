// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialImageDecorator.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/SlateImageRun.h"

TSharedRef< FTutorialImageDecorator > FTutorialImageDecorator::Create()
{
	return MakeShareable( new FTutorialImageDecorator() );
}

FTutorialImageDecorator::FTutorialImageDecorator()
{
}

bool FTutorialImageDecorator::Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const
{
	return ( RunParseResult.Name == TEXT("img") );
}

TSharedRef< ISlateRun > FTutorialImageDecorator::Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style)
{
	const FTextRange* const BrushNameRange = RunParseResult.MetaData.Find( TEXT("src") );

	FTextRange ModelRange;
	ModelRange.BeginIndex = InOutModelText->Len();
	*InOutModelText += TEXT('\x200B'); // Zero-Width Breaking Space
	ModelRange.EndIndex = InOutModelText->Len();

	FRunInfo RunInfo( RunParseResult.Name );
	for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid( Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	if ( BrushNameRange != NULL )
	{
		const FString BrushNameString = OriginalText.Mid(BrushNameRange->BeginIndex, BrushNameRange->EndIndex - BrushNameRange->BeginIndex);

		return FSlateImageRun::Create( RunInfo, InOutModelText, *GetPathToImage(BrushNameString), 0, ModelRange );
	}

	const FInlineTextImageStyle& ImageStyle = FInlineTextImageStyle::GetDefault();
	return FSlateImageRun::Create( RunInfo, InOutModelText, &ImageStyle.Image, ImageStyle.Baseline, ModelRange );
}

FString FTutorialImageDecorator::GetPathToImage(const FString& InSrcMetaData)
{
	// images are content-relative if they are created from a project
	FString FullPath;
	if(FPackageName::IsValidLongPackageName(InSrcMetaData))
	{
		FullPath = FPackageName::LongPackageNameToFilename(InSrcMetaData);
		if(FPaths::GetExtension(FullPath).Len() == 0)
		{
			FullPath += TEXT(".png");
		}
	}
	else
	{
		// otherwise they are raw paths relative to the engine
		FullPath = InSrcMetaData;
		FPaths::MakePathRelativeTo(FullPath, FPlatformProcess::BaseDir());
	}

	return FullPath;
}
