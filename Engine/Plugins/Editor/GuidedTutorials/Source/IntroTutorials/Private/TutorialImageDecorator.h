// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/ITextDecorator.h"

class ISlateStyle;

class FTutorialImageDecorator : public ITextDecorator
{
public:

	static TSharedRef< FTutorialImageDecorator > Create();
	virtual ~FTutorialImageDecorator() {}

public:

	virtual bool Supports( const FTextRunParseResults& RunParseResult, const FString& Text ) const override;

	virtual TSharedRef< ISlateRun > Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override;

	static FString GetPathToImage(const FString& InSrcMetaData);

private:

	FTutorialImageDecorator();
};
