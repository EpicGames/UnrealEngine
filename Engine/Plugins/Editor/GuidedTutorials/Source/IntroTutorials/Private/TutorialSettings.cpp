// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialSettings.h"

UTutorialSettings::UTutorialSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

