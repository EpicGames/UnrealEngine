// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "EditorTutorial.h"
#include "TutorialSettings.generated.h"

/** Per-project tutorial settings */
UCLASS(config=Editor)
class UTutorialSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Categories for tutorials */
	UPROPERTY(Config, EditAnywhere, Category="Tutorials")
	TArray<FTutorialCategory> Categories;

	/** Tutorial to start on project startup */
	UPROPERTY(Config, EditAnywhere, Category="Tutorials", meta=(MetaClass="/Script/IntroTutorials.EditorTutorial"))
	FSoftClassPath StartupTutorial;
};
