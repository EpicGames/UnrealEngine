// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialStateSettings.h"
#include "Templates/SubclassOf.h"
#include "EditorTutorial.h"
#include "Misc/PackageName.h"

UTutorialStateSettings::UTutorialStateSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bDismissedAllTutorials = false;
}

void UTutorialStateSettings::PostInitProperties()
{
	Super::PostInitProperties();

	for(const auto& Progress : TutorialsProgress)
	{
		if (FPackageName::IsValidObjectPath(*Progress.Tutorial.ToString()))
		{
			TSubclassOf<UEditorTutorial> TutorialClass = LoadClass<UEditorTutorial>(NULL, *Progress.Tutorial.ToString(), NULL, LOAD_None, NULL);
			if (TutorialClass != nullptr)
			{
				UEditorTutorial* Tutorial = TutorialClass->GetDefaultObject<UEditorTutorial>();
				ProgressMap.Add(Tutorial, Progress);
			}
		}
	}
}

int32 UTutorialStateSettings::GetProgress(UEditorTutorial* InTutorial, bool& bOutHaveSeenTutorial) const
{
	const FTutorialProgress* FoundProgress = ProgressMap.Find(InTutorial);
	if(FoundProgress != nullptr)
	{
		bOutHaveSeenTutorial = true;
		return FoundProgress->CurrentStage;
	}

	bOutHaveSeenTutorial = false;
	return 0;
}

bool UTutorialStateSettings::HaveSeenTutorial(UEditorTutorial* InTutorial) const
{
	return ProgressMap.Find(InTutorial) != nullptr;
}

bool UTutorialStateSettings::HaveCompletedTutorial(UEditorTutorial* InTutorial) const
{
	const FTutorialProgress* FoundProgress = ProgressMap.Find(InTutorial);
	if (FoundProgress != nullptr)
	{
		return FoundProgress->CurrentStage >= InTutorial->Stages.Num() - 1;
	}
	return false;
}

void UTutorialStateSettings::RecordProgress(UEditorTutorial* InTutorial, int32 CurrentStage)
{
	if(InTutorial != nullptr)
	{
		FTutorialProgress* FoundProgress = ProgressMap.Find(InTutorial);
		if(FoundProgress != nullptr)
		{
			FoundProgress->CurrentStage = FMath::Max(FoundProgress->CurrentStage, CurrentStage);
		}
		else
		{
			FTutorialProgress Progress;
			Progress.Tutorial = FSoftClassPath(InTutorial->GetClass());
			Progress.CurrentStage = CurrentStage;
			Progress.bUserDismissed = false;
			Progress.bUserDismissedThisSession = false;

			ProgressMap.Add(InTutorial, Progress);
		}
	}
}

void UTutorialStateSettings::DismissTutorial(UEditorTutorial* InTutorial, bool bDismissAcrossSessions)
{
	if (InTutorial != nullptr)
	{
		FTutorialProgress* FoundProgress = ProgressMap.Find(InTutorial);
		if (FoundProgress != nullptr)
		{
			FoundProgress->bUserDismissed = bDismissAcrossSessions;
			FoundProgress->bUserDismissedThisSession = true;
		}
		else
		{
			FTutorialProgress Progress;
			Progress.Tutorial = FSoftClassPath(InTutorial->GetClass());
			Progress.CurrentStage = 0;
			Progress.bUserDismissed = bDismissAcrossSessions;
			Progress.bUserDismissedThisSession = true;

			ProgressMap.Add(InTutorial, Progress);
		}
	}
}

bool UTutorialStateSettings::IsTutorialDismissed(UEditorTutorial* InTutorial) const
{
	if (GetMutableDefault<UTutorialStateSettings>()->AreAllTutorialsDismissed())
	{
		return true;
	}

	const FTutorialProgress* FoundProgress = ProgressMap.Find(InTutorial);
	if (FoundProgress != nullptr)
	{
		return FoundProgress->bUserDismissed || FoundProgress->bUserDismissedThisSession;
	}
	return false;
}

void UTutorialStateSettings::DismissAllTutorials()
{
	bDismissedAllTutorials = true;
}

bool UTutorialStateSettings::AreAllTutorialsDismissed()
{
	return bDismissedAllTutorials;
}

void UTutorialStateSettings::SaveProgress()
{
	TutorialsProgress.Empty();

	for(const auto& ProgressMapEntry : ProgressMap)
	{
		TutorialsProgress.Add(ProgressMapEntry.Value);
	}

	SaveConfig();
}

void UTutorialStateSettings::ClearProgress()
{
	ProgressMap.Empty();
	TutorialsProgress.Empty();
	bDismissedAllTutorials = false;

	SaveConfig();
}
