// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "TutorialStateSettings.generated.h"

class UEditorTutorial;

/** Track the progress of an individual tutorial */
USTRUCT()
struct FTutorialProgress
{
	GENERATED_USTRUCT_BODY()

	FTutorialProgress()
		: CurrentStage(0)
		, bUserDismissed(false)
		, bUserDismissedThisSession(false)
	{
	}

	UPROPERTY()
	FSoftClassPath Tutorial;

	UPROPERTY()
	int32 CurrentStage;

	UPROPERTY()
	bool bUserDismissed;

	/** Non-persistent flag indicating the user dismissed this tutorial */
	bool bUserDismissedThisSession;
};

/** Tutorial settings used to track completion state */
UCLASS(config=EditorSettings)
class UTutorialStateSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Config)
	TArray<FTutorialProgress> TutorialsProgress;

	/** UObject interface */
	virtual void PostInitProperties() override;

	/** Reset the progress and completion sate of all tutorials */
	void ClearProgress();

	/** Get the recorded progress of the pass-in tutorial */
	int32 GetProgress(UEditorTutorial* InTutorial, bool& bOutHaveSeenTutorial) const;

	/** Check if we have seen the passed-in tutorial before */
	bool HaveSeenTutorial(UEditorTutorial* InTutorial) const;

	/** Check if completed the passed in tutorial (i.e. seen all of its stages) */
	bool HaveCompletedTutorial(UEditorTutorial* InTutorial) const;

	/** Flag a tutorial as dismissed */
	void DismissTutorial(UEditorTutorial* InTutorial, bool bDismissAcrossSessions);

	/** Check if a tutorial has been dismissed */
	bool IsTutorialDismissed(UEditorTutorial* InTutorial) const;

	/** Record the progress of the passed-in tutorial */
	void RecordProgress(UEditorTutorial* InTutorial, int32 CurrentStage);

	/** Save the progress of all our tutorials */
	void SaveProgress();

	/** Dismiss all tutorials, used by right-click option on scholar cap button (STutorialButton) */
	void DismissAllTutorials();

	/** Returns true if user has dismissed tutorials */
	bool AreAllTutorialsDismissed();

private:
	/** Recorded progress */
	TMap<UEditorTutorial*, FTutorialProgress> ProgressMap;

	/** Record if user has chosen to cancel all tutorials */
	UPROPERTY(Config)
	bool bDismissedAllTutorials;
};
