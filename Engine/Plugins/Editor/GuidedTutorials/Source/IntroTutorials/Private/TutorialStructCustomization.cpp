// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialStructCustomization.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Types/ISlateMetaData.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Docking/TabManager.h"
#include "PropertyHandle.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "DetailWidgetRow.h"
#include "IntroTutorials.h"
#include "EditorTutorial.h"
#include "STutorialEditableText.h"
#include "TutorialMetaData.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "TutorialStructCustomization"

TSharedRef<IPropertyTypeCustomization> FTutorialContentCustomization::MakeInstance()
{
	return MakeShareable( new FTutorialContentCustomization );
}

void FTutorialContentCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> TypeProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContent, Type));
	TSharedPtr<IPropertyHandle> ContentProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContent, Content));
	TSharedPtr<IPropertyHandle> ExcerptNameProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContent, ExcerptName));
	TSharedPtr<IPropertyHandle> TextProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContent, Text));

	// Show and hide various widgets based on current content type
	struct Local
	{
		static EVisibility GetContentVisibility(TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			check(InPropertyHandle.IsValid());

			uint8 Value = 0;
			if(InPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
			{
				const ETutorialContent::Type EnumValue = (ETutorialContent::Type)Value;
				return (EnumValue == ETutorialContent::UDNExcerpt) ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		}

		static EVisibility GetExcerptNameVisibility(TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			check(InPropertyHandle.IsValid());

			uint8 Value = 0;
			if(InPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
			{
				const ETutorialContent::Type EnumValue = (ETutorialContent::Type)Value;
				return (EnumValue == ETutorialContent::UDNExcerpt) ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		}

		static EVisibility GetTextVisibility(TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			check(InPropertyHandle.IsValid());

			uint8 Value = 0;
			if(InPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
			{
				const ETutorialContent::Type EnumValue = (ETutorialContent::Type)Value;
				return (EnumValue == ETutorialContent::Text) ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		}

		static EVisibility GetRichTextVisibility(TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			check(InPropertyHandle.IsValid());

			uint8 Value = 0;
			if(InPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
			{
				const ETutorialContent::Type EnumValue = (ETutorialContent::Type)Value;
				return (EnumValue == ETutorialContent::RichText) ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		}

		static FText GetValueAsText(TSharedPtr<IPropertyHandle> InPropertyHandle)
		{
			FText Text;

			if( InPropertyHandle->GetValueAsFormattedText( Text ) == FPropertyAccess::MultipleValues )
			{
				Text = NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
			}

			return Text;
		}

		static void OnTextCommitted( const FText& NewText, ETextCommit::Type /*CommitInfo*/, TSharedPtr<IPropertyHandle> InPropertyHandle )
		{
			InPropertyHandle->SetValueFromFormattedString( NewText.ToString() );
		}

		static void OnTextChanged( const FText& NewText, TSharedPtr<IPropertyHandle> InPropertyHandle )
		{
			InPropertyHandle->SetValueFromFormattedString( NewText.ToString() );
		}
	};

	HeaderRow
	.NameContent()
	[
		ContentProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			TypeProperty->CreatePropertyValueWidget()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			.Visibility_Static(&Local::GetContentVisibility, TypeProperty)
			+SHorizontalBox::Slot()
			[
				ContentProperty->CreatePropertyValueWidget()
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			.Visibility_Static(&Local::GetExcerptNameVisibility, TypeProperty)
			+SHorizontalBox::Slot()
			[
				ExcerptNameProperty->CreatePropertyValueWidget()
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			.Visibility_Static(&Local::GetTextVisibility, TypeProperty)
			+SHorizontalBox::Slot()
			[
				TextProperty->CreatePropertyValueWidget()
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			.Visibility_Static(&Local::GetRichTextVisibility, TypeProperty)
			+SHorizontalBox::Slot()
			[
				SNew(STutorialEditableText)
				.Text_Static(&Local::GetValueAsText, TextProperty)
				.OnTextCommitted(FOnTextCommitted::CreateStatic(&Local::OnTextCommitted, TextProperty))
				.OnTextChanged(FOnTextChanged::CreateStatic(&Local::OnTextChanged, TextProperty))
			]
		]
	];
}

void FTutorialContentCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{

}

/** 'Tooltip' window to indicate what is currently being picked and how to abort the picker (Esc) */
class SWidgetPickerFloatingWindow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWidgetPickerFloatingWindow){}

	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
	SLATE_ARGUMENT(TSharedPtr<class IPropertyHandle>, FriendlyNameProperty)
	SLATE_ARGUMENT(FName, SpecificWidgetType)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<class IPropertyHandle> InStructPropertyHandle, TSharedRef<class IPropertyHandle> InPickPropertyHandle)
	{
		StructPropertyHandle = InStructPropertyHandle;
		PickPropertyHandle = InPickPropertyHandle;

		ParentWindow = InArgs._ParentWindow;
		SpecificWidgetType = InArgs._SpecificWidgetType;
		FriendlyNameProperty = InArgs._FriendlyNameProperty;
		
		ChildSlot
		[
			SNew(SToolTip)
			.Text(this, &SWidgetPickerFloatingWindow::GetPickerStatusText)
		];
	}
	
	/* Returns the name of the picked widget */
	FName GetPickedWidgetName()
	{
		return PickedWidgetName;
	}

	/** 
	 * Return the name of the given widget (Will filter out widgets that do no match the specific type if applicable
	 *
	 * @param InWidget	The widget to get the pickable name of
	 * @returns Pickable Name of the widget (or None if it doesnt match a specific widget type)
	 *
	 */
	FName GetPickableNameForWidget(TSharedRef<SWidget> InWidget) const
	{
		FName PickableName;
		if (InWidget->GetTag() != NAME_None)
		{
			PickableName = InWidget->GetTag();
		}
		else
		{
			// If we have specified a specific widget to to pick check this one matches
			TSharedPtr<FTagMetaData> WidgetMetaData = InWidget->GetMetaData<FTagMetaData>();
			if (WidgetMetaData.IsValid())
			{
				PickableName = WidgetMetaData->Tag;
			}
			else if (SpecificWidgetType != NAME_None)
			{
				FName TheType = InWidget->GetType();
				if (TheType == SpecificWidgetType)
				{
					if (SpecificWidgetType == FName("SDockTab"))
					{
						TSharedRef<SDockTab> DockTab = StaticCastSharedRef<SDockTab>(InWidget);
						FString TabIdent = DockTab.Get().GetLayoutIdentifier().ToString();
						PickableName = FName(*TabIdent);
					}
				}
			}
		}
		return PickableName;
	}
	
private:
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		PickedWidgetName = NAME_None;
		PickedAllMetaData.Reset();
		
		FWidgetPath Path = FSlateApplication::Get().LocateWindowUnderMouse(FSlateApplication::Get().GetCursorPos(), FSlateApplication::Get().GetInteractiveTopLevelWindows(), true);

		for(int32 PathIndex = Path.Widgets.Num() - 1; PathIndex >= 0; PathIndex--)
		{
			TSharedRef<SWidget> PathWidget = Path.Widgets[PathIndex].Widget;
						
			PickedWidgetName = GetPickableNameForWidget(PathWidget);
			if (PickedWidgetName != NAME_None)
			{
				PickedAllMetaData = PathWidget->GetAllMetaData<FTagMetaData>();
				break;
			}			
		}

		// kind of a hack, but we need to maintain keyboard focus otherwise we wont get our keypress to 'pick'
		FSlateApplication::Get().SetKeyboardFocus(SharedThis(this), EFocusCause::SetDirectly);
		if(ParentWindow.IsValid())
		{
			// also kind of a hack, but this is the only way at the moment to get a 'cursor decorator' without using the drag-drop code path
			ParentWindow.Pin()->MoveWindowTo(FSlateApplication::Get().GetCursorPos() + FSlateApplication::Get().GetCursorSize());
		}
	}
		
	FText GetPickerStatusText() const
	{
		return FText::Format(LOCTEXT("TootipHint", "{0} (Esc to pick)"), FText::FromName(PickedWidgetName));
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Escape)
		{
			if( InKeyEvent.IsLeftControlDown() == false )
			{
				// We cant set a parameter if this isn't valid !
				check(PickPropertyHandle.IsValid());
				PickPropertyHandle->SetValue(PickedWidgetName);

				TSharedPtr<IPropertyHandle> TypeProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, Type));
				TypeProperty->SetValue((uint8)ETutorialAnchorIdentifier::NamedWidget);

				FString FriendlyNameToSet = PickedWidgetName.ToString();

				// Reset the other fields
				StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, GUIDString))->SetValue(FString());
				StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, OuterName))->SetValue(FString());
				StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, TabToFocusOrOpen))->SetValue(FString());

				// Handle custom widget type picks
				if ((SpecificWidgetType.IsValid() == true) && (SpecificWidgetType != NAME_None))
				{
					if (SpecificWidgetType == FName("SDockTab"))
					{
						StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, TabToFocusOrOpen))->SetValue(PickedWidgetName);
					}
				}

				for (const auto& MetaDataEntry : PickedAllMetaData)
				{
					if (MetaDataEntry->IsOfType<FGraphNodeMetaData>())
					{
						TSharedRef<FGraphNodeMetaData> GraphNodeMeta = StaticCastSharedRef<FGraphNodeMetaData>(MetaDataEntry);
						StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, GUIDString))->SetValue(GraphNodeMeta->GUID.ToString());
						StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, OuterName))->SetValue(GraphNodeMeta->OuterName);
						StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, FriendlyName))->SetValue(GraphNodeMeta->FriendlyName);
					}
					else if (MetaDataEntry->IsOfType<FTutorialMetaData>())
					{
						TSharedRef<FTutorialMetaData> TutorialMeta = StaticCastSharedRef<FTutorialMetaData>(MetaDataEntry);
						FriendlyNameToSet = TutorialMeta->FriendlyName;

						// TabTypeToOpen only really applies to specifc widget types, so if we dont have one dont set the parameter
						if ((SpecificWidgetType.IsValid() == true) && (SpecificWidgetType != NAME_None))
						{
							StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, TabToFocusOrOpen))->SetValue(TutorialMeta->TabTypeToOpen);
						}
					}
					else
					{
						TSharedRef<FTagMetaData> GraphNodeMeta = StaticCastSharedRef<FTagMetaData>(MetaDataEntry);
						FriendlyNameToSet = GraphNodeMeta->Tag.ToString();
					}

					// Set the friendly name to the PickedWidget name - we might not have any metadata
					if (FriendlyNameProperty.IsValid())
					{
						FriendlyNameProperty->SetValue(FriendlyNameToSet);
					}
				}
			}
			
			// Reset the pick data
			PickedWidgetName = NAME_None;
			PickedAllMetaData.Reset();

			if(ParentWindow.IsValid())
			{
				FSlateApplication::Get().RequestDestroyWindow(ParentWindow.Pin().ToSharedRef());
				ParentWindow.Reset();
			}

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	/** We need to support keyboard focus to process the 'Esc' key */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

private:
	/** Handle to the property struct (Data other than the name is in here) */
	TSharedPtr<class IPropertyHandle> StructPropertyHandle;

	/** Handle to the name property struct. This is the property we are actually picking */
	TSharedPtr<class IPropertyHandle> PickPropertyHandle;

	/* Handle to the friendly name property we should set if any. */
	TSharedPtr<class IPropertyHandle> FriendlyNameProperty;

	/** Handle to the window that contains this widget */
	TWeakPtr<SWindow> ParentWindow;

	/** The widget name we are picking */
	FName PickedWidgetName;

	/* The metadata for the widget we are picking */
	TArray<TSharedRef<FTagMetaData>> PickedAllMetaData;

	/* If we are we picking a specific widget type this will specify a typename (EG SDockTab) */
	FName SpecificWidgetType;
};

/** Widget used to launch a 'picking' session */
class SWidgetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SWidgetPicker ) {}

	SLATE_ARGUMENT(FName, SpecificWidgetType)

	SLATE_ARGUMENT(TSharedPtr<class IPropertyHandle>, FriendlyNameProperty)

	SLATE_END_ARGS()

	~SWidgetPicker()
	{
		// kill the picker window as well if this widget is going away - that way we dont get dangling refs to the property
		if(PickerWindow.IsValid() && FSlateApplication::IsInitialized())
		{
			FSlateApplication::Get().RequestDestroyWindow(PickerWindow.Pin().ToSharedRef());
			PickerWindow.Reset();
			PickerWidget.Reset();
		}
	}

	void Construct(const FArguments& InArgs, TSharedRef<class IPropertyHandle> InStructPropertyHandle, TSharedRef<class IPropertyHandle> InPickPropertyHandle, IPropertyTypeCustomizationUtils* InStructCustomizationUtils)
	{
		StructPropertyHandle = InStructPropertyHandle;
		PickPropertyHandle = InPickPropertyHandle;

		SpecificWidgetType = InArgs._SpecificWidgetType;
		FriendlyNameProperty = InArgs._FriendlyNameProperty;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SWidgetPicker::HandlePickerStatusText)
				.Font(InStructCustomizationUtils->GetRegularFont())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
				.OnClicked( this, &SWidgetPicker::OnClicked )
				.ContentPadding(4.0f)
				.ForegroundColor( FSlateColor::UseForeground() )
				.IsFocusable(false)
				[
					SNew( SImage )
					.Image( FAppStyle::GetBrush("Icons.EyeDropper") )
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			]
		];
	}

private:
	FReply OnClicked()
	{
		// launch a picker window
		if(!PickerWindow.IsValid())
		{
			TSharedRef<SWindow> NewWindow = SWindow::MakeCursorDecorator();
			NewWindow->MoveWindowTo(FSlateApplication::Get().GetCursorPos());
			PickerWindow = NewWindow;

			NewWindow->SetContent(
				SAssignNew(PickerWidget, SWidgetPickerFloatingWindow, StructPropertyHandle.ToSharedRef(), PickPropertyHandle.ToSharedRef())
				.ParentWindow(NewWindow)
				.SpecificWidgetType(SpecificWidgetType)
				.FriendlyNameProperty(FriendlyNameProperty)
				);

			TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
			if (RootWindow.IsValid())
			{
				FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef());
			}
			else
			{
				FSlateApplication::Get().AddWindow(NewWindow);
			}

			FIntroTutorials& IntroTutorials = FModuleManager::Get().GetModuleChecked<FIntroTutorials>("IntroTutorials");
			IntroTutorials.OnIsPicking().BindSP(this, &SWidgetPicker::OnIsPicking);
			IntroTutorials.OnValidatePickingCandidate().BindSP(this, &SWidgetPicker::OnValidatePickingCandidate);
		}

		return FReply::Handled();
	}

	FText HandlePickerStatusText() const
	{
		if (PickPropertyHandle.IsValid())
		{
			FString WidgetValue;
			PickPropertyHandle->GetValue(WidgetValue);

			if (FriendlyNameProperty.IsValid())
			{
				FString FriendlyName;
				FriendlyNameProperty->GetValue(FriendlyName);
				if (FriendlyName.IsEmpty() == false)
				{
					WidgetValue = FriendlyName;
				}
			}

			return FText::FromString(WidgetValue);
		}
		return FText();
	}

	FText MakeFriendlyStringFromName(const FString& WidgetName) const
	{
		// We will likely have meta data for this eventually. For now just parse a comma delimited string which currently will either be a name or ident,name,UID
		if (WidgetName == "None")
		{
			return FText::FromName(*WidgetName);
		}

		FString FriendlyName;
		StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, FriendlyName))->GetValue(FriendlyName);
		if (FriendlyName.IsEmpty()==false)
		{
			return FText::FromName(*FriendlyName);
		}
		return FText::FromName(*WidgetName);
	}

	bool OnIsPicking(FName& OutWidgetNameToHighlight) const
	{
		if(PickerWidget.IsValid())
		{
			OutWidgetNameToHighlight = PickerWidget.Pin()->GetPickedWidgetName();
			return true;
		}
		
		return false;
	}
		
	bool OnValidatePickingCandidate(TSharedRef<SWidget> InWidget, FName& OutWidgetNameToHighlight, bool& bOutShouldHighlight) const
	{
		bool bIsPicking = false;
		bOutShouldHighlight = false;
		TSharedPtr<FTagMetaData> WidgetMetaData = InWidget->GetMetaData<FTagMetaData>();
		const FName WidgetTag = (WidgetMetaData.IsValid() && WidgetMetaData->Tag.IsValid()) ? WidgetMetaData->Tag : InWidget->GetTag();
		if (PickerWidget.IsValid())
		{
			// Is the given widget a candidate
			if ((WidgetTag != NAME_None) )
			{
				bIsPicking = true;
			}
			if (SpecificWidgetType != NAME_None)
			{
				bIsPicking = InWidget->GetType() == SpecificWidgetType;
			}
			
			// If we are picking a widget check if we should also highlight it
			if (bIsPicking == true)
			{
				OutWidgetNameToHighlight = PickerWidget.Pin()->GetPickedWidgetName();
				FName InPickableName = PickerWidget.Pin()->GetPickableNameForWidget(InWidget);

				if (InPickableName == OutWidgetNameToHighlight)
				{
					bOutShouldHighlight = true;
				}
			}
		}
		return bIsPicking;
	}

private:
	/** Picker window widget */
	TWeakPtr<SWidgetPickerFloatingWindow> PickerWidget;

	/** Picker window */
	TWeakPtr<SWindow> PickerWindow;

	/** Handle to the struct we are customizing */
	TSharedPtr<class IPropertyHandle> StructPropertyHandle;

	/** Handle to the property we are customizing */
	TSharedPtr<class IPropertyHandle> PickPropertyHandle;

	/* Handle to the friendly name property we should set  if any. */
	TSharedPtr<class IPropertyHandle> FriendlyNameProperty;

	/* Are we picking a specific widget type */
	FName	SpecificWidgetType;
};

TSharedRef<IPropertyTypeCustomization> FTutorialContentAnchorCustomization::MakeInstance()
{
	return MakeShareable( new FTutorialContentAnchorCustomization );
}

void FTutorialContentAnchorCustomization::CustomizeHeader( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> DrawHighlightProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, bDrawHighlight));
	TSharedPtr<IPropertyHandle> WidgetNameProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, WrapperIdentifier));
	TSharedPtr<IPropertyHandle> TabToFocusProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, TabToFocusOrOpen));
	TSharedPtr<IPropertyHandle> FriendlyNameProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTutorialContentAnchor, FriendlyName));

	HeaderRow
	.NameContent()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			DrawHighlightProperty->CreatePropertyNameWidget()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			TabToFocusProperty->CreatePropertyNameWidget()
		]
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(500.0f)
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SNew(SWidgetPicker, InStructPropertyHandle, WidgetNameProperty.ToSharedRef(), &StructCustomizationUtils)
			.FriendlyNameProperty(FriendlyNameProperty)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			DrawHighlightProperty->CreatePropertyValueWidget()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f)
		[
			SNew(SWidgetPicker, InStructPropertyHandle, TabToFocusProperty.ToSharedRef(), &StructCustomizationUtils)
				.SpecificWidgetType("SDockTab")
		]
	];
}

void FTutorialContentAnchorCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{

}

#undef LOCTEXT_NAMESPACE
