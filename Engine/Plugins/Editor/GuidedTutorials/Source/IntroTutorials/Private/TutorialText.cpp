// Copyright Epic Games, Inc. All Rights Reserved.

#include "TutorialText.h"
#include "Framework/Text/ITextDecorator.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/TextDecorators.h"
#include "Engine/Blueprint.h"

#include "IIntroTutorials.h"
#include "IntroTutorials.h"
#include "IDocumentation.h"
#include "ISourceCodeAccessor.h"
#include "ISourceCodeAccessModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "EditorTutorial.h"
#include "SourceCodeNavigation.h"
#include "TutorialImageDecorator.h"
#include "EngineAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "TutorialHyperlinkDecorator.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "TutorialText"

TArray<TSharedPtr<FHyperlinkTypeDesc>> FTutorialText::HyperlinkDescs;


/**
 * This is a custom decorator used to allow arbitrary styling of text within a rich-text editor
 * This is required since normal text styling can only work with known styles from a given Slate style-set
 */
class FTextStyleDecorator : public ITextDecorator
{
public:

	static TSharedRef<FTextStyleDecorator> Create()
	{
		return MakeShareable(new FTextStyleDecorator());
	}

	virtual ~FTextStyleDecorator()
	{
	}

	virtual bool Supports(const FTextRunParseResults& RunParseResult, const FString& Text) const override
	{
		return (RunParseResult.Name == TEXT("TextStyle"));
	}

	virtual TSharedRef<ISlateRun> Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef< FString >& InOutModelText, const ISlateStyle* Style) override
	{
		FRunInfo RunInfo(RunParseResult.Name);
		for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
		{
			RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
		}

		FTextRange ModelRange;
		ModelRange.BeginIndex = InOutModelText->Len();
		*InOutModelText += OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex);
		ModelRange.EndIndex = InOutModelText->Len();

		return FSlateTextRun::Create(RunInfo, InOutModelText, FTextStyleAndName::CreateTextBlockStyle(RunInfo), ModelRange);
	}
};


static void OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url)
	{
		if( FEngineAnalytics::IsAvailable() )
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("BrowserLink"), *Url));

			FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.BrowserLinkClicked"), EventAttributes );
		}

		FPlatformProcess::LaunchURL(**Url, nullptr, nullptr);
	}
}

static void OnDocLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url)
	{
		if( FEngineAnalytics::IsAvailable() )
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("DocLink"), *Url));

			FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.DocLinkClicked"), EventAttributes );
		}

		IDocumentation::Get()->Open(*Url, FDocumentationSourceInfo(TEXT("tutorials")));
	}
}

static void ParseTutorialLink(const FString &InternalLink)
{
	UBlueprint* Blueprint = LoadObject<UBlueprint>(nullptr, *InternalLink);
	if (Blueprint && Blueprint->GeneratedClass)
	{
		FIntroTutorials& IntroTutorials = FModuleManager::GetModuleChecked<FIntroTutorials>(TEXT("IntroTutorials"));
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		IntroTutorials.LaunchTutorial(Blueprint->GeneratedClass->GetDefaultObject<UEditorTutorial>(), IIntroTutorials::ETutorialStartType::TST_RESTART, MainFrameModule.GetParentWindow());

		if( FEngineAnalytics::IsAvailable() )
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("TutorialLink"), InternalLink));

			FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.TutorialLinkClicked"), EventAttributes );
		}
	}
}

static void OnTutorialLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url)
	{
		ParseTutorialLink(*Url);
	}
}

static void ParseCodeLink(const FString &InternalLink)
{
	// Tokens used by the code parsing. Details in the parse section	
	static const FString ProjectSpecifier(TEXT("[PROJECTPATH]"));
	static const FString ProjectPathSpecifier(TEXT("[PROJECT]"));
	static const FString EnginePathSpecifier(TEXT("[ENGINEPATH]"));

	FString Path;
	int32 Line = 0;
	int32 Col = 0;

	TArray<FString> Tokens;
	InternalLink.ParseIntoArray(Tokens, TEXT(","), 0);
	int32 TokenStringsCount = Tokens.Num();
	if (TokenStringsCount > 0)
	{
		Path = Tokens[0];
	}
	if (TokenStringsCount > 1)
	{
		TTypeFromString<int32>::FromString(Line, *Tokens[1]);
	}
	if (TokenStringsCount > 2)
	{
		TTypeFromString<int32>::FromString(Col, *Tokens[2]);
	}

	ISourceCodeAccessModule& SourceCodeAccessModule = FModuleManager::LoadModuleChecked<ISourceCodeAccessModule>("SourceCodeAccess");
	ISourceCodeAccessor& SourceCodeAccessor = SourceCodeAccessModule.GetAccessor();

	if (Path.Contains(EnginePathSpecifier) == true)
	{
		// replace engine path specifier with path to engine
		Path.ReplaceInline(*EnginePathSpecifier, *FPaths::EngineDir());
	}

	if (Path.Contains(ProjectSpecifier) == true)
	{
		// replace project specifier with path to project
		Path.ReplaceInline(*ProjectSpecifier, FApp::GetProjectName());
	}

	if (Path.Contains(ProjectPathSpecifier) == true)
	{
		// replace project specifier with path to project
		Path.ReplaceInline(*ProjectPathSpecifier, *FPaths::GetProjectFilePath());
	}

	Path = FPaths::ConvertRelativePathToFull(Path);

	SourceCodeAccessor.OpenFileAtLine(Path, Line, Col);

	if( FEngineAnalytics::IsAvailable() )
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CodeLink"), InternalLink));

		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.CodeLinkClicked"), EventAttributes );
	}
}

static void OnCodeLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url)
	{	
		ParseCodeLink(*Url);
	}
}

static void ParseAssetLink(const FString& InternalLink, const FString* Action)
{
	UObject* RequiredObject = LoadObject<UObject>(nullptr, *InternalLink);
	if (RequiredObject != nullptr)
	{
		if(Action && *Action == TEXT("select"))
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			TArray<UObject*> AssetToBrowse;
			AssetToBrowse.Add(RequiredObject);
			ContentBrowserModule.Get().SyncBrowserToAssets(AssetToBrowse);
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RequiredObject);
		}

		if( FEngineAnalytics::IsAvailable() )
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("AssetLink"), InternalLink));

			FEngineAnalytics::GetProvider().RecordEvent( TEXT("Rocket.Tutorials.AssetLinkClicked"), EventAttributes );
		}
	}
}

static void OnAssetLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	const FString* Action = Metadata.Find(TEXT("action"));
	if(Url)
	{
		ParseAssetLink(*Url, Action);
	}
}

static TSharedRef<IToolTip> OnGenerateDocTooltip(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	FText DisplayText;
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url != nullptr)
	{
		DisplayText = FText::Format(LOCTEXT("DocLinkPattern", "View Documentation: {0}"), FText::FromString(*Url));
	}
	else
	{
		DisplayText = LOCTEXT("UnknownLink", "Empty Hyperlink");
	}

	FString UrlString;
	if(Url != nullptr)
	{
		UrlString = *Url;
	}

	// urls for rich tooltips must start with Shared/
	if(UrlString.Len() > 0 && !UrlString.StartsWith(TEXT("Shared")))
	{
		UrlString = FString(TEXT("Shared")) / UrlString;
	}

	FString ExcerptString;
	const FString* Excerpt = Metadata.Find(TEXT("excerpt"));
	if(Excerpt != nullptr)
	{
		ExcerptString = *Excerpt;
	}

	return IDocumentation::Get()->CreateToolTip(DisplayText, nullptr, UrlString, ExcerptString);
}

static FText OnGetAssetTooltipText(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url != nullptr)
	{
		const FString* Action = Metadata.Find(TEXT("action"));
		return FText::Format(LOCTEXT("AssetLinkPattern", "{0} asset: {1}"), (Action == nullptr || *Action == TEXT("select")) ? LOCTEXT("AssetOpenDesc", "Open") : LOCTEXT("AssetFindDesc", "Find"), FText::FromString(*Url));
	}

	return LOCTEXT("InvalidAssetLink", "Invalid Asset Link");
}

static FText OnGetCodeTooltipText(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url != nullptr)
	{
		const bool bUseShortIDEName = true;
		return FText::Format(LOCTEXT("CodeLinkPattern", "Open code in {0}: {1}"), FSourceCodeNavigation::GetSelectedSourceCodeIDE(), FText::FromString(*Url));
	}

	return LOCTEXT("InvalidCodeLink", "Invalid Code Link");
}

static FText OnGetTutorialTooltipText(const FSlateHyperlinkRun::FMetadata& Metadata)
{
	const FString* Url = Metadata.Find(TEXT("href"));
	if(Url != nullptr)
	{
		return FText::Format(LOCTEXT("TutorialLinkPattern", "Open tutorial: {0}"), FText::FromString(*Url));
	}

	return LOCTEXT("InvalidTutorialLink", "Invalid Tutorial Link");
}

void FTutorialText::GetRichTextDecorators(bool bForEditing, TArray< TSharedRef< class ITextDecorator > >& OutDecorators)
{
	Initialize();

	if(bForEditing)
	{
		for(const auto& HyperlinkDesc : HyperlinkDescs)
		{
			OutDecorators.Add(FHyperlinkDecorator::Create(HyperlinkDesc->Id, HyperlinkDesc->OnClickedDelegate, HyperlinkDesc->TooltipTextDelegate, HyperlinkDesc->TooltipDelegate));
		}
	}
	else
	{
		for(const auto& HyperlinkDesc : HyperlinkDescs)
		{
			OutDecorators.Add(FTutorialHyperlinkDecorator::Create(HyperlinkDesc->Id, HyperlinkDesc->OnClickedDelegate, HyperlinkDesc->TooltipTextDelegate, HyperlinkDesc->TooltipDelegate));
		}	
	}

	OutDecorators.Add(FTextStyleDecorator::Create());
	OutDecorators.Add(FTutorialImageDecorator::Create());
}

void FTutorialText::Initialize()
{
	if(HyperlinkDescs.Num() == 0)
	{
		HyperlinkDescs = {
			MakeShareable(new FHyperlinkTypeDesc(
				EHyperlinkType::Browser, 
				LOCTEXT("BrowserLinkTypeLabel", "URL"), 
				LOCTEXT("BrowserLinkTypeTooltip", "A link that opens a browser window (e.g. http://www.unrealengine.com)"),
				TEXT("browser"),
				FSlateHyperlinkRun::FOnClick::CreateStatic(&OnBrowserLinkClicked))),

			MakeShareable(new FHyperlinkTypeDesc(
				EHyperlinkType::UDN, 
				LOCTEXT("UDNLinkTypeLabel", "UDN"), 
				LOCTEXT("UDNLinkTypeTooltip", "A link that opens some UDN documentation (e.g. /Engine/Blueprints/UserGuide/Types/ClassBlueprint)"),
				TEXT("udn"),
				FSlateHyperlinkRun::FOnClick::CreateStatic(&OnDocLinkClicked), 
				FSlateHyperlinkRun::FOnGetTooltipText(),
				FSlateHyperlinkRun::FOnGenerateTooltip::CreateStatic(&OnGenerateDocTooltip))),

			MakeShareable(new FHyperlinkTypeDesc(
				EHyperlinkType::Asset, 
				LOCTEXT("AssetLinkTypeLabel", "Asset"), 
				LOCTEXT("AssetLinkTypeTooltip", "A link that opens an asset (e.g. /Game/StaticMeshes/SphereMesh.SphereMesh)"),
				TEXT("asset"),
				FSlateHyperlinkRun::FOnClick::CreateStatic(&OnAssetLinkClicked),
				FSlateHyperlinkRun::FOnGetTooltipText::CreateStatic(&OnGetAssetTooltipText))),

			MakeShareable(new FHyperlinkTypeDesc(
				EHyperlinkType::Code, 
				LOCTEXT("CodeLinkTypeLabel", "Code"), 
				LOCTEXT("CodeLinkTypeTooltip", "A link that opens code in your selected IDE.\nFor example: [PROJECTPATH]/Private/SourceFile.cpp,1,1.\nThe numbers correspond to line number and column number.\nYou can use [PROJECT], [PROJECTPATH] and [ENGINEPATH] tags to make paths."),
				TEXT("code"),
				FSlateHyperlinkRun::FOnClick::CreateStatic(&OnCodeLinkClicked),
				FSlateHyperlinkRun::FOnGetTooltipText::CreateStatic(&OnGetCodeTooltipText))),

			MakeShareable(new FHyperlinkTypeDesc(
				EHyperlinkType::Tutorial, 
				LOCTEXT("TutorialLinkTypeLabel", "Tutorial"), 
				LOCTEXT("TutorialLinkTypeTooltip", "A type of asset link that opens another tutorial, e.g. /Game/Tutorials/StaticMeshTutorial.StaticMeshTutorial"),
				TEXT("tutorial"),
				FSlateHyperlinkRun::FOnClick::CreateStatic(&OnTutorialLinkClicked),
				FSlateHyperlinkRun::FOnGetTooltipText::CreateStatic(&OnGetTutorialTooltipText)))
		};
	}
}

const TArray<TSharedPtr<FHyperlinkTypeDesc>>& FTutorialText::GetHyperLinkDescs()
{
	Initialize();

	return HyperlinkDescs;
}

#undef LOCTEXT_NAMESPACE
