// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Styling/AppStyle.h"
#include "EditorTutorialStyle.h"

namespace EHyperlinkType
{
	enum Type
	{
		Browser,
		UDN,
		Tutorial,
		Code,
		Asset
	};
}

/** Helper struct to hold info about hyperlink types */
struct FHyperlinkTypeDesc
{
	FHyperlinkTypeDesc(EHyperlinkType::Type InType, const FText& InText, const FText& InTooltipText, const FString& InId, FSlateHyperlinkRun::FOnClick InOnClickedDelegate, FSlateHyperlinkRun::FOnGetTooltipText InTooltipTextDelegate = FSlateHyperlinkRun::FOnGetTooltipText(), FSlateHyperlinkRun::FOnGenerateTooltip InTooltipDelegate = FSlateHyperlinkRun::FOnGenerateTooltip())
		: Type(InType)
		, Id(InId)
		, Text(InText)
		, TooltipText(InTooltipText)
		, OnClickedDelegate(InOnClickedDelegate)
		, TooltipTextDelegate(InTooltipTextDelegate)
		, TooltipDelegate(InTooltipDelegate)
	{
	}

	/** The type of the link */
	EHyperlinkType::Type Type;

	/** Tag used by this hyperlink's run */
	FString Id;

	/** Text to display in the UI */
	FText Text;

	/** Tooltip text to display in the UI */
	FText TooltipText;

	/** Delegate to execute for this hyperlink's run */
	FSlateHyperlinkRun::FOnClick OnClickedDelegate;

	/** Delegate used to retrieve the text to display in the hyperlink's tooltip */
	FSlateHyperlinkRun::FOnGetTooltipText TooltipTextDelegate;

	/** Delegate used to generate hyperlink's tooltip */
	FSlateHyperlinkRun::FOnGenerateTooltip TooltipDelegate;
};


/** Text style and name to display in the UI */
struct FTextStyleAndName
{
	/** Legacy: Flags controlling which TTF or OTF font should be picked from the given font family */
	struct EFontStyle
	{
		typedef uint8 Flags;
		enum Flag
		{
			Regular = 0,
			Bold = 1<<0,
			Italic = 1<<1,
		};
	};

	FTextStyleAndName(FName InStyle, FText InDisplayName)
		: Style(InStyle)
		, DisplayName(InDisplayName)
	{}

	FRunInfo CreateRunInfo() const
	{
		FRunInfo RunInfo(TEXT("TextStyle"));
		RunInfo.MetaData.Add(TEXT("Style"), Style.ToString());
		return RunInfo;
	}

	static FName GetStyleFromRunInfo(const FRunInfo& InRunInfo)
	{
		const FString* const StyleString = InRunInfo.MetaData.Find(TEXT("Style"));
		if(StyleString)
		{
			return **StyleString;
		}
		else
		{
			// legacy styling support - try to map older flexible styles to new fixed styles

			int32 FontSize = 11;
			const FString* const FontSizeString = InRunInfo.MetaData.Find(TEXT("FontSize"));
			if(FontSizeString)
			{
				FontSize = static_cast<uint8>(FPlatformString::Atoi(**FontSizeString));
			}

			if(FontSize > 24)
			{
				return TEXT("Tutorials.Content.HeaderText2");
			}
			else if(FontSize > 11 && FontSize <= 24)
			{
				return TEXT("Tutorials.Content.HeaderText1");
			}
			else
			{
				EFontStyle::Flags FontStyle = EFontStyle::Regular;
				const FString* const FontStyleString = InRunInfo.MetaData.Find(TEXT("FontStyle"));
				if(FontStyleString)
				{
					if(*FontStyleString == TEXT("Bold"))
					{
						FontStyle = EFontStyle::Bold;
					}
					else if(*FontStyleString == TEXT("Italic"))
					{
						FontStyle = EFontStyle::Italic;
					}
					else if(*FontStyleString == TEXT("BoldItalic"))
					{
						FontStyle = EFontStyle::Bold | EFontStyle::Italic;
					}
				}

				FLinearColor FontColor = FLinearColor::Black;
				const FString* const FontColorString = InRunInfo.MetaData.Find(TEXT("FontColor"));
				if(FontColorString && !FontColor.InitFromString(*FontColorString))
				{
					FontColor = FLinearColor::Black;
				}

				if(FontStyle != EFontStyle::Regular || FontColor != FLinearColor::Black)
				{
					return TEXT("Tutorials.Content.TextBold");
				}
				else
				{
					return TEXT("Tutorials.Content.Text");
				}
			}
		}
	}

	FTextBlockStyle CreateTextBlockStyle() const
	{
		return FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>(Style);
	}

	static FTextBlockStyle CreateTextBlockStyle(const FRunInfo& InRunInfo)
	{
		return FEditorTutorialStyle::Get().GetWidgetStyle<FTextBlockStyle>(GetStyleFromRunInfo(InRunInfo));
	}

	/** The style identifier */
	FName Style;

	/** The text to display for this style */
	FText DisplayName;
};



/** Helper functions for generating rich text */
struct FTutorialText
{
public:
	static void GetRichTextDecorators(bool bForEditing, TArray<TSharedRef<class ITextDecorator>>& OutDecorators);

	static const TArray<TSharedPtr<FHyperlinkTypeDesc>>& GetHyperLinkDescs();

private:
	static void Initialize();

private:
	static TArray<TSharedPtr<FHyperlinkTypeDesc>> HyperlinkDescs;
};
