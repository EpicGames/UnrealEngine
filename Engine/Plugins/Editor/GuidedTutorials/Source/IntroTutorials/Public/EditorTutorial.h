// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Types/SlateEnums.h"
#include "UObject/SoftObjectPath.h"
#include "EditorTutorial.generated.h"

class AActor;
class UTexture2D;

/** The type of tutorial content to display */
UENUM(meta=(ScriptName="TutorialContentType"))
namespace ETutorialContent
{
	enum Type : int
	{
		/** Blank - displays no content */
		None,

		/** Plain text content */
		Text,

		/** Content from a UDN excerpt */
		UDNExcerpt,

		/** Rich text content */
		RichText,
	};
}

/** The type of tutorial content to display */
UENUM()
namespace ETutorialAnchorIdentifier
{
	enum Type : int
	{
		/** No anchor */
		None,

		/** Uses a tutorial wrapper widget */
		NamedWidget,

		/** An asset accessible via the content browser */
		Asset,
	};
}

/** Category description */
USTRUCT()
struct INTROTUTORIALS_API FTutorialCategory
{
	GENERATED_USTRUCT_BODY()

	/** Period-separated category name, e.g. "Editor Quickstart.Level Editor" */
	UPROPERTY(EditAnywhere, Category="Content")
	FString Identifier;

	/** Title of the category */
	UPROPERTY(EditAnywhere, Category="Content")
	FText Title;

	/** Sort order, used by the tutorial browser - set in editor game-agnostic INI file */
	UPROPERTY()
	int32 SortOrder = INDEX_NONE;

	/** Localized text to use to describe this category */
	UPROPERTY(EditAnywhere, Category="Content", meta=(MultiLine=true))
	FText Description;

	/** Icon for this tutorial, used when presented to the user in the tutorial browser. Only used if there isn't a valid texture to use. */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	FString Icon;

	/** Texture for this tutorial, used when presented to the user in the tutorial browser. */
	UPROPERTY(EditAnywhere, Category="Tutorial", meta=(AllowedClasses="/Script/Engine.Texture2D"))
	FSoftObjectPath Texture;
};

/** Content wrapper */
USTRUCT()
struct INTROTUTORIALS_API FTutorialContent
{
	GENERATED_USTRUCT_BODY()

	FTutorialContent()
	{
		Type = ETutorialContent::RichText;
	}

	/** The type of this content */
	UPROPERTY(EditAnywhere, Category="Content")
	TEnumAsByte<ETutorialContent::Type> Type;

	/** Content reference string, path etc. */
	UPROPERTY(EditAnywhere, Category="Content")
	FString Content;

	/** Excerpt name for UDN excerpt */
	UPROPERTY(EditAnywhere, Category="Content")
	FString ExcerptName;

	/** Localized text to use with this content */
	UPROPERTY(EditAnywhere, Category="Content", meta=(MultiLine=true))
	FText Text;
};

/** A way of identifying something to be highlighted by a tutorial */
USTRUCT()
struct INTROTUTORIALS_API FTutorialContentAnchor
{
	GENERATED_USTRUCT_BODY()
	
	FTutorialContentAnchor()
	{
		 Type = ETutorialAnchorIdentifier::None;
		 bDrawHighlight = true;
	}

	UPROPERTY(EditAnywhere, Category="Anchor")
	TEnumAsByte<ETutorialAnchorIdentifier::Type> Type;

	/** If widget is in a wrapper widget, this is the wrapper widget name */
	UPROPERTY(EditAnywhere, Category="Anchor")
	FName WrapperIdentifier;

	/** If reference is an asset, we use this to resolve it */
	UPROPERTY(EditAnywhere, Category="Anchor")
	FSoftObjectPath Asset;

	/** Whether to draw an animated highlight around the widget */
	UPROPERTY(EditAnywhere, Category="Anchor")
	bool bDrawHighlight;
	
	/* Tab on which to focus (EG 'My Blueprint' tab). */
	UPROPERTY(EditAnywhere, Category = "Anchor")
	FString TabToFocusOrOpen;

	//FBlueprintGraphNodeMetaData specific members	
	/* User friendly name to display in the dialog */
	UPROPERTY(EditAnywhere, Category = "AnchorMeta|FBlueprintGraphNodeMetaData")
	FString FriendlyName;

	/* The GUID string */
	UPROPERTY(EditAnywhere, Category = "AnchorMeta|FBlueprintGraphNodeMetaData")
 	FString GUIDString;
	
	/* Name of the outer object - should be the blueprint that 'owns' the node */
	UPROPERTY(EditAnywhere, Category = "AnchorMeta|FBlueprintGraphNodeMetaData")
	FString OuterName;
};

/** Content that is displayed relative to a widget */
USTRUCT()
struct INTROTUTORIALS_API FTutorialWidgetContent
{
	GENERATED_USTRUCT_BODY()

	FTutorialWidgetContent()
	{
		ContentWidth = 350.0f;
		HorizontalAlignment = HAlign_Center;
		VerticalAlignment = VAlign_Bottom;
		Offset = FVector2D::ZeroVector;
		bAutoFocus = false;
	}

	/** Content to associate with widget */
	UPROPERTY(EditAnywhere, Category="Widget")
	FTutorialContent Content;

	/** Anchor for content widget to highlight */
	UPROPERTY(EditAnywhere, Category="Widget")
	FTutorialContentAnchor WidgetAnchor;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Widget")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Widget")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

	/** Custom offset from widget */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Widget")
	FVector2D Offset;

	/** Content width - text will be wrapped at this point */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Widget", meta=(UIMin="10.0", UIMax="600.0"))
	float ContentWidth;

	/** If this a node that can be focused (EG a blueprint node) should we auto focus on it */
	UPROPERTY(EditAnywhere, Category = "Anchor")
	bool bAutoFocus;
};

/** A single tutorial stage, containing the optional main content & a number of widgets with content attached */
USTRUCT()
struct INTROTUTORIALS_API FTutorialStage
{
	GENERATED_USTRUCT_BODY()

	/** Identifier for this stage */
	UPROPERTY(EditAnywhere, Category="Stage")
	FName Name;

	/** Non-widget-bound content to display in this stage */
	UPROPERTY(EditAnywhere, Category="Stage")
	FTutorialContent Content;

	/** Widget-bound content to display for this stage */
	UPROPERTY(EditAnywhere, Category="Stage")
	TArray<FTutorialWidgetContent> WidgetContent;

	/** Text to display on the next button */
	UPROPERTY(EditAnywhere, Category="Stage")
	FText NextButtonText;
	
	/** Text to display on the back button */
	UPROPERTY(EditAnywhere, Category = "Stage")
	FText BackButtonText;

	/** List of platforms to test against. Meaning of test is determined by InvertPlatformTest. */
	UPROPERTY(EditAnywhere, Category = "Stage")
	TArray<FString> PlatformsToTest;

	/** If false, stage will be skipped if running on any platform in PlatformsToTest. If true, the stage will be if not running on any platform in PlatformsToTest. */
	UPROPERTY(EditAnywhere, Category = "Stage")
	bool bInvertPlatformTest = true;
};

/** An asset used to build a stage-by-stage tutorial in the editor */
UCLASS(Blueprintable, hideCategories=(Object), EditInlineNew)
class INTROTUTORIALS_API UEditorTutorial : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Title of this tutorial, used when presented to the user */
	UPROPERTY(EditAnywhere, Category="Tutorial", AssetRegistrySearchable)
	FText Title;

	/** Sorting priority, used by the tutorial browser */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = "Tutorial")
	int32 SortOrder=0;

	/** Icon name for this tutorial, used when presented to the user in the tutorial browser. This is a name for the icon in the Slate editor style. Only used if there isn't a valid texture to use. */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	FString Icon;

	/** Texture for this tutorial, used when presented to the user in the tutorial browser. */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	TObjectPtr<UTexture2D> Texture;

	/** Category of this tutorial, used to organize tutorials when presented to the user */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	FString Category;

	/** Content to be displayed for this tutorial when presented to the user in summary */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	FTutorialContent SummaryContent;

	/** The various stages of this tutorial */
	UPROPERTY(EditAnywhere, Category="Stages")
	TArray<FTutorialStage> Stages;

	/** Tutorial to optionally chain back to if the "back" button is clicked on the first stage */
	UPROPERTY(EditAnywhere, Category = "Tutorial", meta = (MetaClass = "/Script/IntroTutorials.EditorTutorial"))
	FSoftClassPath PreviousTutorial;

	/** Tutorial to optionally chain onto after this tutorial completes */
	UPROPERTY(EditAnywhere, Category="Tutorial", meta=(MetaClass="/Script/IntroTutorials.EditorTutorial"))
	FSoftClassPath NextTutorial;

	/**
	* Attempts to find the actor specified by PathToActor in the current editor world
	* @param	PathToActor	The path to the actor (e.g. PersistentLevel.PlayerStart)
	* @return	A reference to the actor, or none if it wasn't found
	*/
	UFUNCTION(BlueprintPure, Category = "Tutorial")
	AActor* GetActorReference(FString PathToActor);

	/** A standalone tutorial displays no navigation buttons and each content widget has a close button */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	bool bIsStandalone;
	
	/** Asset to open & attach the tutorial to. Non-widget-bound content will appear in the asset's window */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	FSoftObjectPath AssetToUse;

	/** The path this tutorial was imported from, if any. */
	UPROPERTY()
	FString ImportPath;

	/** Hide this tutorial in the tutorials browser */
	UPROPERTY(EditAnywhere, Category="Tutorial")
	bool bHideInBrowser;

	/** UObject implementation */
	virtual UWorld* GetWorld() const override;

	/** Comma seperated list of tags the search will use to help find this tutorial  */
	UPROPERTY(EditAnywhere, Category = "Tutorial", AssetRegistrySearchable)
	FText SearchTags;

public:
	/** Called when a tutorial stage is started */
	void HandleTutorialStageStarted(FName StageName);

	/** Called when a tutorial stage ends */
	void HandleTutorialStageEnded(FName StageName);

	/** Called each tick so the Blueprint can optionally complete or skip stages */
	void HandleTickCurrentStage(FName StageName);

	/** Called when a tutorial is launched */
	void HandleTutorialLaunched();

	/** Called when a tutorial is closed */
	void HandleTutorialClosed();

protected:
	/** Event fired when a tutorial stage begins */
	UFUNCTION(BlueprintImplementableEvent, Category = "Tutorial")
	void OnTutorialStageStarted(FName StageName);

	/** Event fired when a tutorial stage ends */
	UFUNCTION(BlueprintImplementableEvent, Category = "Tutorial")
	void OnTutorialStageEnded(FName StageName);

	/** Event fired when a tutorial is launched */
	UFUNCTION(BlueprintImplementableEvent, Category = "Tutorial")
	void OnTutorialLaunched();

	/** Event fired when a tutorial is closed */
	UFUNCTION(BlueprintImplementableEvent, Category = "Tutorial")
	void OnTutorialClosed();

	/** Advance to the next stage of a tutorial */
	UFUNCTION(BlueprintCallable, Category="Tutorial")
	static void GoToNextTutorialStage();

	/** Advance to the previous stage of a tutorial */
	UFUNCTION(BlueprintCallable, Category="Tutorial")
	static void GoToPreviousTutorialStage();

	/** Begin a tutorial. Note that this will end the current tutorial that is in progress, if any */
	UFUNCTION(BlueprintCallable, Category="Tutorial")
	static void BeginTutorial(UEditorTutorial* TutorialToStart, bool bRestart);

	/** 
	 * Open an asset for use by a tutorial
	 * @param	Asset	The asset to open
	 */
	UFUNCTION(BlueprintCallable, Category="Tutorial")
	static void OpenAsset(UObject* Asset);

	/**
	 * Sets the visibility of the engine folder in the content browser
	 */
	UFUNCTION(BlueprintCallable, Category="Tutorial")
	static void SetEngineFolderVisibilty(bool bNewVisibility);

	/**
	 * Returns the visibility of the engine folder in the content browser
	 */
	UFUNCTION(BlueprintCallable, Category="Tutorial")
	static bool GetEngineFolderVisibilty();
};
