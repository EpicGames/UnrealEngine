// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class SWindow;
class UEditorTutorial;
struct FTutorialCategory;

/**
 * The public interface to this module.  In most cases, this interface is only public to sibling modules 
 * within this plugin.
 */
class IIntroTutorials : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IIntroTutorials& Get()
	{
		static const FName ModuleName = "IntroTutorials";
		return FModuleManager::LoadModuleChecked< IIntroTutorials >( ModuleName );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "IntroTutorials";
		return FModuleManager::Get().IsModuleLoaded( ModuleName );
	}

	/**
	 * Launch a tutorial with the specified asset.
	 *
	 * @param TutorialAssetName The name of the tutorial asset.
	 */
	virtual void LaunchTutorial(const FString& TutorialAssetName, TSharedPtr<SWindow> InNavigationWindow = nullptr ) = 0;

	//Parameter for LaunchTutorial function
	enum ETutorialStartType
	{
		//Default start type, picks up where user left off if appropriate
		TST_CONTINUE,
		//Restart and clear progress
		TST_RESTART,
		//Start at the last stage of the tutorial, used when backing up to a previous tutorial
		TST_LASTSTAGE
	};

	/**
	 * Launch a tutorial immediately, bypassing the tutorial browser.
	 * 
	 * @param	Tutorial	The tutorial to launch
	 * @param	bRestart	Whether to restart the tutorial or resume from where we left off last time.
	 * @param	InNavigationWindow	Optional window to launch the tutorial from - this is where navigation will be displayed.
	 */
	virtual void LaunchTutorial(UEditorTutorial* Tutorial, IIntroTutorials::ETutorialStartType InStartType = TST_RESTART, TWeakPtr<SWindow> InNavigationWindow = nullptr, FSimpleDelegate OnTutorialClosed = FSimpleDelegate(), FSimpleDelegate OnTutorialExited = FSimpleDelegate()) = 0;

	/**
 	 *  Close all tutorial content, including the browser.
	 */
	virtual void CloseAllTutorialContent() = 0;

	/**
	 * Create a widget that allows access to the tutorial for the current context.
	 * @param	InContext			The name of the context this widget is attached to (e.g. "LevelEditor")
	 * @param	InContextWindow		The window that the context is attached to (e.g. the main window, or an asset editor tab)
	 * @return a widget used to access context-sensitive tutorials
	 */
	virtual TSharedRef<SWidget> CreateTutorialsWidget(FName InContext, TWeakPtr<SWindow> InContextWindow = nullptr) const = 0;

	/**
	* Create a widget that shows a "loading" visual while waiting for tutorials to load.
	* @param	InContextWindow		The window that the context is attached to (e.g. the main window, or an asset editor tab)
	* @return a widget used to access context-sensitive tutorials
	*/
	virtual TSharedPtr<SWidget> CreateTutorialsLoadingWidget(TWeakPtr<SWindow> InContextWindow = nullptr) const = 0;

	/**
	* Register a new category with the tutorial browser. Categories are automatically created when tutorial blueprints are discovered, 
	* but this will allow you to apply some customization for tutorial categories defined by plugins
	* @param	InTutorialCategory	The new category
	*/
	virtual void RegisterCategory(FTutorialCategory NewCategory) = 0;
};

